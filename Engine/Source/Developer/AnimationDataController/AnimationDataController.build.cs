// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AnimationDataController : ModuleRules
	{
        public AnimationDataController(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
                }
             );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"MovieScene"
				}
			);
			
			if (Target.bCompileAgainstEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{													
						"UnrealEd",
						"CurveEditor"
					}
				);
			}
		}
	}
}
