// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDataController.h"
#include "Animation/Skeleton.h"
#include "AnimDataControllerActions.h"
#include "ChangeTransactor.h"
#include "MovieSceneTimeHelpers.h"

#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/AnimSequence.h"

#include "Algo/Transform.h"
#include "Animation/AnimationSettings.h"
#include "UObject/NameTypes.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDataController)

#define LOCTEXT_NAMESPACE "AnimDataController"

#if WITH_EDITOR
void UAnimDataController::SetModel(TScriptInterface<IAnimationDataModel> InModel)
{	
	if (Model != nullptr)
	{
		Model->GetModifiedEvent().RemoveAll(this);
	}

	ModelInterface = InModel;
	Model = CastChecked<UAnimDataModel>(InModel.GetObject(), ECastCheckedType::NullAllowed);
	
	ChangeTransactor.SetTransactionObject(Model.Get());
}

void UAnimDataController::OpenBracket(const FText& InTitle, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	
	if (BracketDepth == 0)
	{
		bShouldTransactBracket = bShouldTransact;
	}

	if (UE::FChangeTransactor::CanTransactChanges() && !ChangeTransactor.IsTransactionPending() && ShouldTransact(bShouldTransact))
	{
		ChangeTransactor.OpenTransaction(InTitle);

		ConditionalAction<UE::Anim::FCloseBracketAction>(bShouldTransact, InTitle.ToString());
	}

	if (BracketDepth == 0)
	{
		FBracketPayload Payload;
		Payload.Description = InTitle.ToString();

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketOpened, Payload);
	}

	++BracketDepth;
}

void UAnimDataController::CloseBracket(bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (BracketDepth == 0)
	{
		Report(ELogVerbosity::Error, LOCTEXT("NoExistingBracketError", "Attempt to close bracket that was not previously opened"));
		return;
	}

	--BracketDepth;

	if (BracketDepth == 0)
	{
		if (UE::FChangeTransactor::CanTransactChanges() && ShouldTransact(bShouldTransact && bShouldTransactBracket))
		{
			ensure(ChangeTransactor.IsTransactionPending());

			ConditionalAction<UE::Anim::FOpenBracketAction>(bShouldTransact, TEXT("Open Bracket"));

			ChangeTransactor.CloseTransaction();
		}
		
		Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketClosed);

		bShouldTransactBracket = false;
	}
}

void UAnimDataController::SetNumberOfFrames(FFrameNumber Length, bool bShouldTransact)
{
	ValidateModel();
	const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();

	const int32 DeltaFrames = FMath::Abs(Length.Value - CurrentNumberOfFrames.Value);
	
	const FFrameNumber T0 = Length > CurrentNumberOfFrames ? CurrentNumberOfFrames : CurrentNumberOfFrames - DeltaFrames;
	const FFrameNumber T1 = Length > CurrentNumberOfFrames ? Length : CurrentNumberOfFrames;

	ResizeNumberOfFrames(Length, T0, T1, bShouldTransact);
}

void UAnimDataController::ResizeNumberOfFrames(FFrameNumber NewLength, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact)
{
	ValidateModel();
	
	const TRange<FFrameNumber> PlayRange(TRange<FFrameNumber>::BoundsType::Inclusive(0), TRange<FFrameNumber>::BoundsType::Exclusive(FMath::Max(1, Model->GetNumberOfKeys())));
	if (NewLength >= 0)
	{
		if (NewLength != Model->GetNumberOfFrames())
		{
			// Ensure that T0 is within the current play range
			if (PlayRange.Contains(T0))
			{
				// Ensure that the start and end length of either removal or insertion are valid
				if (T0 < T1)
				{
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("ResizePlayLength", "Resizing Play Length"), bShouldTransact);

					const FFrameRate CurrentFrameRate = Model->GetFrameRate();
					const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();

					FSequenceLengthChangedPayload Payload;
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Payload.T0 = static_cast<float>(CurrentFrameRate.AsSeconds(T0));
					Payload.T1 = static_cast<float>(CurrentFrameRate.AsSeconds(T1));
					Payload.PreviousLength = static_cast<float>(CurrentFrameRate.AsSeconds(CurrentNumberOfFrames));
					PRAGMA_ENABLE_DEPRECATION_WARNINGS

					Payload.PreviousNumberOfFrames = CurrentNumberOfFrames;
					Payload.Frame0 = T0;
					Payload.Frame1 = T1;

					ConditionalAction<UE::Anim::FResizePlayLengthInFramesAction>(bShouldTransact, Model.Get(), Payload.Frame0, Payload.Frame1);

					Model->NumberOfFrames = NewLength.Value;
					Model->NumberOfKeys = Model->NumberOfFrames + 1;
	
					Model->GetNotifier().Notify<FSequenceLengthChangedPayload>(EAnimDataModelNotifyType::SequenceLengthChanged, Payload);
				}
				else
				{
					ReportErrorf(LOCTEXT("InvalidEndTimeError", "Invalid T1, smaller that T0 value: T0 {0}, T1 {1}"), FText::AsNumber(T0.Value), FText::AsNumber(T0.Value));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("InvalidStartTimeError", "Invalid T0, not within existing play range: T0 {0}, Play Length {1}"), FText::AsNumber(T0.Value), FText::AsNumber(Model->GetPlayLength()));
			}
		}
		else if (Model->bPopulated)
		{
			Reportf(ELogVerbosity::Display,LOCTEXT("SamePlayLengthWarning", "New play length is same as existing one: {0} frames"), FText::AsNumber(NewLength.Value));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("InvalidPlayLengthError", "Invalid play length value provided: {0} frames"), FText::AsNumber(NewLength.Value));
	}
}

void UAnimDataController::ResizeInFrames(FFrameNumber NewLength, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact)
{
	ValidateModel();
	
	const int32 CurrentNumberOFrames = Model->GetNumberOfFrames();
	
	const TRange<FFrameNumber> PlayRange(TRange<FFrameNumber>::BoundsType::Inclusive(0), TRange<FFrameNumber>::BoundsType::Exclusive(FMath::Max(1,Model->GetNumberOfKeys())));
	if (NewLength >= 0)
	{
		if (NewLength != Model->GetNumberOfFrames())
		{
			// Ensure that T0 is within the current play range
			if (PlayRange.Contains(T0))
			{
				// Ensure that the start and end length of either removal or insertion are valid
				if (T0 < T1)
				{
					FBracket Bracket = ConditionalBracket(LOCTEXT("ResizeModel", "Resizing Animation Data"), bShouldTransact);

					const bool bInserted = NewLength > CurrentNumberOFrames;
					ResizeNumberOfFrames(NewLength, T0, T1, bShouldTransact);
					
					const FFrameRate CurrentFrameRate = Model->GetFrameRate();
					ResizeCurves(static_cast<float>(CurrentFrameRate.AsSeconds(NewLength)), bInserted, static_cast<float>(CurrentFrameRate.AsSeconds(T0)), static_cast<float>(CurrentFrameRate.AsSeconds(T1)), bShouldTransact);
					ResizeAttributes(static_cast<float>(CurrentFrameRate.AsSeconds(NewLength)), bInserted, static_cast<float>(CurrentFrameRate.AsSeconds(T0)), static_cast<float>(CurrentFrameRate.AsSeconds(T1)), bShouldTransact);
				}
				else
                {
                	ReportErrorf(LOCTEXT("InvalidEndTimeError", "Invalid T1, smaller that T0 value: T0 {0}, T1 {1}"), FText::AsNumber(T0.Value), FText::AsNumber(T1.Value));
                }
            }
            else
            {
                ReportErrorf(LOCTEXT("InvalidStartTimeError", "Invalid T0, not within existing play range: T0 {0}, Play Length {1}"), FText::AsNumber(T0.Value), FText::AsNumber(CurrentNumberOFrames));
            }			
        }
		else if (Model->bPopulated)
        {
           Reportf(ELogVerbosity::Display,LOCTEXT("SameGetPlayLengthWarning", "New play length is same as existing one: {0} frames"), FText::AsNumber(CurrentNumberOFrames));
        }
    }
    else
    {
        ReportErrorf(LOCTEXT("InvalidGetPlayLengthError", "Invalid play length value provided: {0} frames"), FText::AsNumber(CurrentNumberOFrames));
    }
}

void UAnimDataController::SetPlayLength(float Length, bool bShouldTransact /*= true*/)
{
	SetNumberOfFrames(ConvertSecondsToFrameNumber(Length), bShouldTransact);
}

void UAnimDataController::Resize(float Length, float T0, float T1, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	
	ResizeInFrames(ConvertSecondsToFrameNumber(Length), ConvertSecondsToFrameNumber(T0), ConvertSecondsToFrameNumber(T1), bShouldTransact);
}

void UAnimDataController::SetFrameRate(FFrameRate FrameRate, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	// Disallow invalid frame-rates, or 0.0 intervals
	const double FrameRateInterval = FrameRate.AsInterval();
	if ( FrameRate.IsValid() && !FMath::IsNearlyZero(FrameRateInterval) && FrameRateInterval > 0.0)
	{
		// Need to verify framerate
		const FFrameRate CurrentFrameRate = Model->GetFrameRate();
		if (FrameRate.IsMultipleOf(CurrentFrameRate) || FrameRate.IsFactorOf(CurrentFrameRate) || !Model->bPopulated)
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetFrameRate", "Setting Frame Rate"), bShouldTransact);
			ConditionalAction<UE::Anim::FSetFrameRateAction>(bShouldTransact, Model.Get());

			const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();
			const FFrameTime ConvertedLastFrameTime = FFrameRate::TransformTime(CurrentNumberOfFrames, CurrentFrameRate, FrameRate);
			ensure(FMath::IsNearlyZero(ConvertedLastFrameTime.GetSubFrame()) || !Model->bPopulated);
			
			Model->FrameRate = FrameRate;
			Model->NumberOfFrames = ConvertedLastFrameTime.GetFrame().Value;
			Model->NumberOfKeys = Model->NumberOfFrames + 1;
			
			FFrameRateChangedPayload Payload;
			Payload.PreviousFrameRate = CurrentFrameRate;
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::FrameRateChanged, Payload);
		}
		else
        {
        	ReportErrorf(LOCTEXT("NonCompatibleFrameRateError", "Incompatible frame rate provided: {0} not a multiple or fact or {1}"), FrameRate.ToPrettyText(), CurrentFrameRate.ToPrettyText());
        }
	}
	else
	{
		ReportErrorf(LOCTEXT("InvalidFrameRateError", "Invalid frame rate provided: {0}"), FrameRate.ToPrettyText());
	}
}

bool UAnimDataController::RemoveBoneTracksMissingFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	if (Skeleton)
	{
		TArray<FName> TracksToBeRemoved;
		TArray<FName> TracksUpdated;
		const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();

		for (FBoneAnimationTrack& Track : Model->BoneAnimationTracks)
		{
			// Try find correct bone index
			const int32 BoneIndex = ReferenceSkeleton.FindBoneIndex(Track.Name);

			if (BoneIndex != INDEX_NONE && BoneIndex != Track.BoneTreeIndex)
			{
				// Update bone index
				Track.BoneTreeIndex = BoneIndex;
				TracksUpdated.Add(Track.Name);
			}
			else if (BoneIndex == INDEX_NONE)
			{				
				// Remove track
				TracksToBeRemoved.Add(Track.Name);
				Reportf(ELogVerbosity::Display, LOCTEXT("InvalidBoneIndexWarning", "Unable to find bone index, animation track will be removed: {0}"), FText::FromName(Track.Name));				
			}			
		}

		if (TracksToBeRemoved.Num() || TracksUpdated.Num())
		{
			FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveBoneTracksMissingFromSkeleton", "Validating Bone Animation Track Data against Skeleton"), bShouldTransact);
			for (const FName& TrackName : TracksToBeRemoved)
			{
				RemoveBoneTrack(TrackName, bShouldTransact);
			}

			for (const FName& TrackName : TracksUpdated)
			{
				FAnimationTrackChangedPayload Payload;
				Payload.Name = TrackName;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);
			}
		}

		return TracksToBeRemoved.Num() > 0 || TracksUpdated.Num() > 0;
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidSkeletonError", "Invalid USkeleton supplied"));
	}

	return false;
}

void UAnimDataController::UpdateAttributesFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact)
{
	ValidateModel();
	
	if (Skeleton)
	{
		// Verifying that bone (names) for attribute data exist on new skeleton
		TArray<FAnimationAttributeIdentifier> ToRemoveIdentifiers;
		TArray<TPair<FAnimationAttributeIdentifier, int32>> ToDuplicateIdentifiers;
			    
		for (const FAnimatedBoneAttribute& Attribute : Model->AnimatedBoneAttributes)
		{
			const int32 NewBoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(Attribute.Identifier.GetBoneName());

			if (NewBoneIndex == INDEX_NONE)
			{
				ToRemoveIdentifiers.Add(Attribute.Identifier);
			}
			else if(NewBoneIndex != Attribute.Identifier.GetBoneIndex())
			{						
				ToDuplicateIdentifiers.Add(TPair<FAnimationAttributeIdentifier, int32>(Attribute.Identifier, NewBoneIndex));
			}
		}

		if (ToRemoveIdentifiers.Num() || ToDuplicateIdentifiers.Num())
		{
			FBracket Bracket = ConditionalBracket(LOCTEXT("VerifyAttributeBoneNames", "Remapping Animation Attribute Data"), bShouldTransact);
			for (const FAnimationAttributeIdentifier& Identifier : ToRemoveIdentifiers)
			{
				RemoveAttribute(Identifier, bShouldTransact);
			}
			
			for (const TPair<FAnimationAttributeIdentifier, int32>& Pair : ToDuplicateIdentifiers)
			{
				const FAnimationAttributeIdentifier& DuplicateIdentifier = Pair.Key;
				FAnimationAttributeIdentifier NewIdentifier(DuplicateIdentifier.GetName(), Pair.Value, DuplicateIdentifier.GetBoneName(), DuplicateIdentifier.GetType());

				DuplicateAttribute(Pair.Key, NewIdentifier, bShouldTransact);
				RemoveAttribute(Pair.Key, bShouldTransact);
			}	
		}
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidSkeletonError", "Invalid USkeleton supplied"));
	}
}

void UAnimDataController::ResetModel(bool bShouldTransact /*= true*/)
{
	ValidateModel();

	FBracket Bracket = ConditionalBracket(LOCTEXT("ResetModel", "Clearing Animation Data"), bShouldTransact);

	if (Model->GetOuter()->IsA<UAnimSequence>())
	{
		RemoveAllBoneTracks(bShouldTransact);
	}

	RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float, bShouldTransact);
	RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform, bShouldTransact);
	RemoveAllAttributes(bShouldTransact);


	SetFrameRate(UAnimationSettings::Get()->GetDefaultFrameRate(), bShouldTransact);
	SetNumberOfFrames(1, bShouldTransact);

	Model->GetNotifier().Notify(EAnimDataModelNotifyType::Reset);
}

bool UAnimDataController::AddCurve(const FAnimationCurveIdentifier& CurveId, int32 CurveFlags /*= EAnimAssetCurveFlags::AACF_Editable*/, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	if (CurveId.CurveName != NAME_None)
	{
		if (IsSupportedCurveType(CurveId.CurveType))
		{
			if (!Model->FindCurve(CurveId))
			{
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddRawCurve", "Adding Animation Curve"), bShouldTransact);

				FCurveAddedPayload Payload;
				Payload.Identifier = CurveId;
				
				auto AddNewCurve = [CurveName = CurveId.CurveName, CurveFlags](auto& CurveTypeArray)
				{
					CurveTypeArray.Add({ CurveName, CurveFlags});
				};
				
				switch (CurveId.CurveType)
				{
				case ERawCurveTrackTypes::RCT_Transform:
					AddNewCurve(Model->CurveData.TransformCurves);
					break;
				case ERawCurveTrackTypes::RCT_Float:
					AddNewCurve(Model->CurveData.FloatCurves);
					break;
				case ERawCurveTrackTypes::RCT_Vector:
				default:
					const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
					ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
				}

				ConditionalAction<UE::Anim::FRemoveCurveAction>(bShouldTransact, CurveId);
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveAdded, Payload);

				return true;
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
				ReportWarningf(LOCTEXT("ExistingCurveNameWarning", "Curve with name {0} and type {1} ({2}) already exists"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
			}			
		}
		else 
		{
			const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
			ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
		}		
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
		ReportWarningf(LOCTEXT("InvalidCurveIdentifierWarning", "Invalid curve identifier provided: name: {0}, type: {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

bool UAnimDataController::DuplicateCurve(const FAnimationCurveIdentifier& CopyCurveId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	ERawCurveTrackTypes SupportedCurveType = CopyCurveId.CurveType;

	if (CopyCurveId.CurveName != NAME_None && NewCurveId.CurveName != NAME_None)
	{
		if (IsSupportedCurveType(SupportedCurveType))
		{
			if (CopyCurveId.CurveType == NewCurveId.CurveType)
			{
				if (Model->FindCurve(CopyCurveId))
				{
					if (!Model->FindCurve(NewCurveId))
					{
						FTransaction Transaction = ConditionalTransaction(LOCTEXT("CopyRawCurve", "Duplicating Animation Curve"), bShouldTransact);

						auto DuplicateCurve = [NewCurveName = NewCurveId.CurveName](auto& CurveDataArray, const auto& SourceCurve)
						{
							auto& DuplicatedCurve = CurveDataArray.Add_GetRef( { NewCurveName, SourceCurve.GetCurveTypeFlags() });
							DuplicatedCurve.CopyCurve(SourceCurve);
						};
						
						switch (SupportedCurveType)
						{
						case ERawCurveTrackTypes::RCT_Transform:
							DuplicateCurve(Model->CurveData.TransformCurves, Model->GetTransformCurve(CopyCurveId));
							break;
						case ERawCurveTrackTypes::RCT_Float:
							DuplicateCurve(Model->CurveData.FloatCurves, Model->GetFloatCurve(CopyCurveId));
							break;
						case ERawCurveTrackTypes::RCT_Vector:
						default:
							const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
							ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
						}

						FCurveAddedPayload Payload;
						Payload.Identifier = NewCurveId;
						Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveAdded, Payload);

						ConditionalAction<UE::Anim::FRemoveCurveAction>(bShouldTransact, NewCurveId);

						return true;
					}
					else
					{
						const FString CurveTypeAsString = GetCurveTypeValueName(NewCurveId.CurveType);
						ReportWarningf(LOCTEXT("ExistingCurveNameWarning", "Curve with name {0} and type {1} ({2}) already exists"), FText::FromName(NewCurveId.CurveName), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(NewCurveId.CurveType)));
					}
				}
				else
				{
					const FString CurveTypeAsString = GetCurveTypeValueName(CopyCurveId.CurveType);
					ReportWarningf(LOCTEXT("CurveNameToDuplicateNotFoundWarning", "Could not find curve with name {0} and type {1} ({2}) for duplication"), FText::FromName(NewCurveId.CurveName), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(NewCurveId.CurveType)));
				}
			}
		}
		else
		{
			const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
			ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("DuplicateInvalidCurveIdentifierWarning", "Invalid curve identifier provided for duplicate: copy: {0}, new: {1}"), FText::FromName(CopyCurveId.CurveName), FText::FromName(NewCurveId.CurveName));
	}

	return false;
}


bool UAnimDataController::RemoveCurve(const FAnimationCurveIdentifier& CurveId, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;

	if (CurveId.CurveName != NAME_None)
	{
		if (IsSupportedCurveType(CurveId.CurveType))
		{
			if (Model->FindCurve(CurveId) != nullptr)
			{
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveCurve", "Removing Animation Curve"), bShouldTransact);

				switch (SupportedCurveType)
				{
					case ERawCurveTrackTypes::RCT_Transform:
					{
						const FTransformCurve& TransformCurve = Model->GetTransformCurve(CurveId);
						ConditionalAction<UE::Anim::FAddTransformCurveAction>(bShouldTransact, CurveId, TransformCurve.GetCurveTypeFlags(), TransformCurve);
						Model->CurveData.TransformCurves.RemoveAll([Name = TransformCurve.GetName()](const FTransformCurve& ToRemoveCurve) { return ToRemoveCurve.GetName() == Name; });
						break;
					}
					case ERawCurveTrackTypes::RCT_Float:
					{
						const FFloatCurve& FloatCurve = Model->GetFloatCurve(CurveId);
						ConditionalAction<UE::Anim::FAddFloatCurveAction>(bShouldTransact, CurveId, FloatCurve.GetCurveTypeFlags(), FloatCurve.FloatCurve.GetConstRefOfKeys(), FloatCurve.Color);
						Model->CurveData.FloatCurves.RemoveAll([Name = FloatCurve.GetName()](const FFloatCurve& ToRemoveCurve) { return ToRemoveCurve.GetName() == Name; });
						break;
					}
					case ERawCurveTrackTypes::RCT_Vector:
					default:
						const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
						ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
				}

				FCurveRemovedPayload Payload;
				Payload.Identifier = CurveId;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveRemoved, Payload);

				return true;
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
				ReportWarningf(LOCTEXT("UnableToFindCurveForRemovalWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
			}
		}
		else
		{
			const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
			ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
		}
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
		ReportWarningf(LOCTEXT("InvalidCurveIdentifierWarning", "Invalid curve identifier provided: name: {0}, type: {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

void UAnimDataController::RemoveAllCurvesOfType(ERawCurveTrackTypes SupportedCurveType /*= ERawCurveTrackTypes::RCT_Float*/, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	FBracket Bracket = ConditionalBracket(LOCTEXT("DeleteAllRawCurve", "Deleting All Animation Curve"), bShouldTransact);
	switch (SupportedCurveType)
	{
	case ERawCurveTrackTypes::RCT_Transform:
	{
		TArray<FTransformCurve> TransformCurves = Model->CurveData.TransformCurves;
		for (const FTransformCurve& Curve : TransformCurves)
		{
			RemoveCurve(FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Transform), bShouldTransact);
		}
		break;
	}
	case ERawCurveTrackTypes::RCT_Float:
	{
		TArray<FFloatCurve> FloatCurves = Model->CurveData.FloatCurves;
		for (const FFloatCurve& Curve : FloatCurves)
		{
			RemoveCurve(FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Float), bShouldTransact);
		}
		break;
	}
	case ERawCurveTrackTypes::RCT_Vector:
	default:
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
	}

}

bool UAnimDataController::SetCurveFlag(const FAnimationCurveIdentifier& CurveId, EAnimAssetCurveFlags Flag, bool bState /*= true*/, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;

	FAnimCurveBase* Curve = nullptr;

	if (SupportedCurveType == ERawCurveTrackTypes::RCT_Float)
	{
		Curve = Model->FindMutableFloatCurveById(CurveId);
	}
	else if (SupportedCurveType == ERawCurveTrackTypes::RCT_Transform)
	{
		Curve = Model->FindMutableTransformCurveById(CurveId);
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
	}
	
	if (Curve)
	{
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetCurveFlag", "Setting Raw Curve Flag"), bShouldTransact);

		const int32 CurrentFlags = Curve->GetCurveTypeFlags();

		ConditionalAction<UE::Anim::FSetCurveFlagsAction>(bShouldTransact, CurveId, CurrentFlags, SupportedCurveType);

		FCurveFlagsChangedPayload Payload;
		Payload.Identifier = CurveId;
		Payload.OldFlags = Curve->GetCurveTypeFlags();

		Curve->SetCurveTypeFlag(Flag, bState);

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveFlagsChanged, Payload);

		return true;
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

bool UAnimDataController::SetCurveFlags(const FAnimationCurveIdentifier& CurveId, int32 Flags, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	FAnimCurveBase* Curve = nullptr;

	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;
	if (SupportedCurveType == ERawCurveTrackTypes::RCT_Float)
	{
		Curve = Model->FindMutableFloatCurveById(CurveId);
	}
	else if (SupportedCurveType == ERawCurveTrackTypes::RCT_Transform)
	{
		Curve = Model->FindMutableTransformCurveById(CurveId);
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
	}

	if (Curve)
	{
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetRawCurveFlag", "Setting Raw Curve Flags"), bShouldTransact);

		const int32 CurrentFlags = Curve->GetCurveTypeFlags();

		ConditionalAction<UE::Anim::FSetCurveFlagsAction>(bShouldTransact, CurveId, CurrentFlags, SupportedCurveType);

		FCurveFlagsChangedPayload Payload;
		Payload.Identifier = CurveId;
		Payload.OldFlags = Curve->GetCurveTypeFlags();

		Curve->SetCurveTypeFlags(Flags);

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveFlagsChanged, Payload);

		return true;
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("UnableToFindCurveForRemovalWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

bool UAnimDataController::SetTransformCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FTransform>& TransformValues, const TArray<float>& TimeKeys, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (TransformValues.Num() == TimeKeys.Num())
	{
		if (Model->FindMutableTransformCurveById(CurveId) != nullptr)
		{
			FBracket Bracket = ConditionalBracket(LOCTEXT("SetTransformCurveKeys_Bracket", "Setting Transform Curve Keys"), bShouldTransact);
			
			struct FKeys
			{
				FKeys(int32 NumKeys)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						ChannelKeys[ChannelIndex].SetNum(NumKeys);
					}
				}

				TArray<FRichCurveKey> ChannelKeys[3];
			};

			FKeys TranslationKeys(TransformValues.Num());
			FKeys RotationKeys(TransformValues.Num());
			FKeys ScaleKeys(TransformValues.Num());

			FKeys* SubCurveKeys[3] = { &TranslationKeys, &RotationKeys, &ScaleKeys };

			// Generate the curve keys
			for (int32 KeyIndex = 0; KeyIndex < TransformValues.Num(); ++KeyIndex)
			{
				const FTransform& Value = TransformValues[KeyIndex];
				const float& Time = TimeKeys[KeyIndex];

				const FVector Translation = Value.GetLocation();
				const FVector Rotation = Value.GetRotation().Euler();
				const FVector Scale = Value.GetScale3D();

				auto SetKey = [Time, KeyIndex](FKeys& Key, const FVector& Vector)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						Key.ChannelKeys[ChannelIndex][KeyIndex] = FRichCurveKey(Time, static_cast<float>(Vector[ChannelIndex]));
					}
				};

				SetKey(TranslationKeys, Translation);
				SetKey(RotationKeys, Rotation);
				SetKey(ScaleKeys, Scale);
			}
			
			for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
			{
				const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
				const FKeys* CurveKeys = SubCurveKeys[SubCurveIndex];
				for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
					FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
					UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
					SetCurveKeys(TargetCurveIdentifier, CurveKeys->ChannelKeys[ChannelIndex], bShouldTransact);
				}
			}

			return true;
		}
		else
		{
			ReportWarningf(LOCTEXT("UnableToFindTransformCurveWarning", "Unable to find transform curve: {0}"), FText::FromName(CurveId.CurveName));
		}
	}
	else
	{
		// time value mismatch
		ReportWarningf(LOCTEXT("InvalidNumberOfTimeAndKeyEntriesWarning", "Number of times and key entries do not match: number of time values {0}, number of key values {1}"), FText::AsNumber(TimeKeys.Num()), FText::AsNumber(TransformValues.Num()));
	}

	return false;	
}


bool UAnimDataController::SetTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, const FTransform& Value, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (Model->FindMutableTransformCurveById(CurveId) != nullptr)
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("AddTransformCurveKey_Bracket", "Setting Transform Curve Key"), bShouldTransact);
		struct FKeys
		{
			FRichCurveKey ChannelKeys[3];
		};

		FKeys VectorKeys[3];
		
		// Generate the rich curve keys		
		const FVector Translation = Value.GetLocation();
		const FVector Rotation = Value.GetRotation().Euler();
		const FVector Scale = Value.GetScale3D();

		auto SetKey = [Time](FKeys& Key, const FVector& Vector)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				Key.ChannelKeys[ChannelIndex] = FRichCurveKey(Time, static_cast<float>(Vector[ChannelIndex]));
			}
		};

		SetKey(VectorKeys[0], Translation);
		SetKey(VectorKeys[1], Rotation);
		SetKey(VectorKeys[2], Scale);
		
		for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
		{
			const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
			const FKeys& VectorCurveKeys = VectorKeys[SubCurveIndex];
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
				FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
				UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
				SetCurveKey(TargetCurveIdentifier, VectorCurveKeys.ChannelKeys[ChannelIndex], bShouldTransact);
			}
		}

		return true;
	}
	else
	{
		ReportWarningf(LOCTEXT("UnableToFindTransformCurveWarning", "Unable to find transform curve: {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}


bool UAnimDataController::RemoveTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (Model->FindMutableTransformCurveById(CurveId))
	{
		const FString BaseCurveName = CurveId.CurveName.ToString();
		const TArray<FString> SubCurveNames = { TEXT( "Translation"), TEXT( "Rotation"), TEXT( "Scale") };
		const TArray<FString> ChannelCurveNames = { TEXT("X"), TEXT("Y"), TEXT("Z") };

		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveTransformCurveKey_Bracket", "Deleting Animation Transform Curve Key"), bShouldTransact);
		
		for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
		{
			const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
				FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
				UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
				RemoveCurveKey(TargetCurveIdentifier, Time, bShouldTransact);
			}
		}

		return true;
	}
	else
	{
		ReportWarningf(LOCTEXT("UnableToFindTransformCurveWarning", "Unable to find transform curve: {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}

bool UAnimDataController::RenameCurve(const FAnimationCurveIdentifier& CurveToRenameId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (NewCurveId.IsValid())
	{
		if (CurveToRenameId != NewCurveId)
		{
			if (CurveToRenameId.CurveType == NewCurveId.CurveType)
			{
				if (FAnimCurveBase* Curve = Model->FindMutableCurveById(CurveToRenameId))
				{
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("RenameCurve", "Renaming Curve"), bShouldTransact);

					FCurveRenamedPayload Payload;
					Payload.Identifier = FAnimationCurveIdentifier(Curve->GetName(), CurveToRenameId.CurveType);

					Curve->SetName(NewCurveId.CurveName);
					Payload.NewIdentifier = NewCurveId;

					ConditionalAction<UE::Anim::FRenameCurveAction>(bShouldTransact, NewCurveId, CurveToRenameId);

					Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveRenamed, Payload);

					return true;
				}
				else
				{
					const FString CurveTypeAsString = GetCurveTypeValueName(CurveToRenameId.CurveType);
					ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveToRenameId.CurveName), FText::FromString(CurveTypeAsString));
				}
			}
			else
			{
				const FString CurrentCurveTypeAsString = GetCurveTypeValueName(CurveToRenameId.CurveType);
				const FString NewCurveTypeAsString = GetCurveTypeValueName(NewCurveId.CurveType);
				ReportWarningf(LOCTEXT("MismatchOfCurveTypesWarning", "Different curve types provided between current and new curve names: {0} ({1}) and {2} ({3})"), FText::FromName(CurveToRenameId.CurveName), FText::FromString(CurrentCurveTypeAsString),
					FText::FromName(NewCurveId.CurveName), FText::FromString(NewCurveTypeAsString));
			}
		}
		else
		{
			ReportWarningf(LOCTEXT("MatchingCurveNamesWarning", "Provided curve names are the same: {0}"), FText::FromName(CurveToRenameId.CurveName));
		}
		
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidCurveIdentiferProvidedWarning", "Invalid new curve identifier provided: {0}"), FText::FromName(NewCurveId.CurveName));
	}

	return false;
}

bool UAnimDataController::SetCurveColor(const FAnimationCurveIdentifier& CurveId, FLinearColor Color, bool bShouldTransact)
{
	ValidateModel();

	if (CurveId.IsValid())
	{
		if (CurveId.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			if (FFloatCurve* Curve = Model->FindMutableFloatCurveById(CurveId))
			{
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("ChangingCurveColor", "Changing Curve Color"), bShouldTransact);

				ConditionalAction<UE::Anim::FSetCurveColorAction>(bShouldTransact, CurveId, Curve->Color);

				Curve->Color = Color;

				FCurveChangedPayload Payload;
				Payload.Identifier = CurveId;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveColorChanged, Payload);

				return true;				
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
				ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
			}
		}
		else
		{
			Report(ELogVerbosity::Warning, LOCTEXT("NonSupportedCurveColorSetWarning", "Changing curve color is currently only supported for float curves"));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidCurveIdentifier", "Invalid Curve Identifier : {0}"), FText::FromName(CurveId.CurveName));
	}	

	return false;
}

bool UAnimDataController::SetCurveComment(const FAnimationCurveIdentifier& CurveId, const FString& Comment, bool bShouldTransact)
{
	ValidateModel();

	if (CurveId.IsValid())
	{
		if (CurveId.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			if (FFloatCurve* Curve = Model->FindMutableFloatCurveById(CurveId))
			{
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("ChangingCurveComment", "Changing Curve Comment"), bShouldTransact);

				ConditionalAction<UE::Anim::FSetCurveCommentAction>(bShouldTransact, CurveId, Curve->Comment);

				Curve->Comment = Comment;

				FCurveChangedPayload Payload;
				Payload.Identifier = CurveId;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveCommentChanged, Payload);

				return true;
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
				ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
			}
		}
		else
		{
			Report(ELogVerbosity::Warning, LOCTEXT("NonSupportedCurveCommentSetWarning", "Changing curve comment is currently only supported for float curves"));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidCurveIdentifier", "Invalid Curve Identifier : {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}

bool UAnimDataController::ScaleCurve(const FAnimationCurveIdentifier& CurveId, float Origin, float Factor, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;
	if (SupportedCurveType == ERawCurveTrackTypes::RCT_Float)
	{
		if (FFloatCurve* Curve = Model->FindMutableFloatCurveById(CurveId))
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("ScalingCurve", "Scaling Curve"), bShouldTransact);

			Curve->FloatCurve.ScaleCurve(Origin, Factor);

			FCurveScaledPayload Payload;
			Payload.Identifier = CurveId;
			Payload.Factor = Factor;
			Payload.Origin = Origin;
			
			ConditionalAction<UE::Anim::FScaleCurveAction>(bShouldTransact, CurveId, Origin, 1.0f / Factor, SupportedCurveType);

			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveScaled, Payload);

			return true;
		}
		else
		{
			ReportWarningf(LOCTEXT("UnableToFindFloatCurveWarning", "Unable to find float curve: {0}"), FText::FromName(CurveId.CurveName));
		}
	}
	else
	{
		Report(ELogVerbosity::Warning, LOCTEXT("NonSupportedCurveScalingWarning", "Scaling curves is currently only supported for float curves"));
	}

	return false;
}

bool UAnimDataController::SetCurveKey(const FAnimationCurveIdentifier& CurveId, const FRichCurveKey& Key, bool bShouldTransact)
{
	ValidateModel();

	if (FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId))
	{
		FCurveChangedPayload Payload;
		Payload.Identifier = CurveId;

		// Set or add rich curve value
		const FKeyHandle Handle = RichCurve->FindKey(Key.Time);
		if (Handle != FKeyHandle::Invalid())
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetNamedCurveKey", "Setting Curve Key"), bShouldTransact);
			// Cache old value for action
			const FRichCurveKey CurrentKey = RichCurve->GetKey(Handle);
			ConditionalAction<UE::Anim::FSetRichCurveKeyAction>(bShouldTransact, CurveId, CurrentKey);

			// Set the new value
			RichCurve->SetKeyValue(Handle, Key.Value);

			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);
		}
		else
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddNamedCurveKey", "Adding Curve Key"), bShouldTransact);
			ConditionalAction<UE::Anim::FRemoveRichCurveKeyAction>(bShouldTransact, CurveId, Key.Time);

			// Add the new key
			RichCurve->AddKey(Key.Time, Key.Value);

			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);
		}

		return true;
	}

	return false;
}

bool UAnimDataController::RemoveCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact)
{
	ValidateModel();

	if (FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId))
	{
		FCurveChangedPayload Payload;
		Payload.Identifier = CurveId;

		// Remove key at time value		
		const FKeyHandle Handle = RichCurve->FindKey(Time);
		if (Handle != FKeyHandle::Invalid())
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveNamedCurveKey", "Removing Curve Key"), bShouldTransact);

			// Cached current value for action
			const FRichCurveKey CurrentKey = RichCurve->GetKey(Handle);
			ConditionalAction<UE::Anim::FAddRichCurveKeyAction>(bShouldTransact, CurveId, CurrentKey);

			RichCurve->DeleteKey(Handle);

			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);

			return true;
		}
		else
		{
			ReportErrorf(LOCTEXT("RichCurveKeyNotFoundError", "Unable to find rich curve key: curve name {0}, time {1}"), FText::FromName(CurveId.CurveName), FText::AsNumber(Time));
		}
	}

	return false;
}


bool UAnimDataController::SetCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FRichCurveKey>& CurveKeys, bool bShouldTransact)
{
	ValidateModel();

	if (FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId))
	{
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingNamedCurveKeys", "Setting Curve Keys"), bShouldTransact);
		ConditionalAction<UE::Anim::FSetRichCurveKeysAction>(bShouldTransact, CurveId, RichCurve->GetConstRefOfKeys());

		// Set rich curve values
		RichCurve->SetKeys(CurveKeys);

		FCurveChangedPayload Payload;
		Payload.Identifier = CurveId;
		Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);

		return true;
	}

	return false;
}


bool UAnimDataController::SetCurveAttributes(const FAnimationCurveIdentifier& CurveId, const FCurveAttributes& Attributes, bool bShouldTransact)
{
	ValidateModel();

	FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId);
	if (RichCurve)
	{
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingNamedCurveAttributes", "Setting Curve Attributes"), bShouldTransact);

		FCurveAttributes CurrentAttributes;
		CurrentAttributes.SetPreExtrapolation(RichCurve->PreInfinityExtrap);
		CurrentAttributes.SetPostExtrapolation(RichCurve->PostInfinityExtrap);		
		ConditionalAction<UE::Anim::FSetRichCurveAttributesAction>(bShouldTransact, CurveId, CurrentAttributes);

		if(Attributes.HasPreExtrapolation())
		{
			RichCurve->PreInfinityExtrap = Attributes.GetPreExtrapolation();
		}

		if(Attributes.HasPostExtrapolation())
		{
			RichCurve->PostInfinityExtrap = Attributes.GetPostExtrapolation();
		}		

		FCurveChangedPayload Payload;
		Payload.Identifier = CurveId;
		Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);

		return true;
	}

	return false;
}

void UAnimDataController::NotifyPopulated()
{
	ValidateModel();

	Model->bPopulated = true;
	Model->GetNotifier().Notify(EAnimDataModelNotifyType::Populated);
}

void UAnimDataController::NotifyBracketOpen()
{
	ValidateModel();
	Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketOpened);
}

void UAnimDataController::NotifyBracketClosed()
{
	ValidateModel();
	Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketClosed);
}

void UAnimDataController::ResizePlayLength(float Length, float T0, float T1, bool bShouldTransact)
{
	ResizeNumberOfFrames(ConvertSecondsToFrameNumber(Length), ConvertSecondsToFrameNumber(T0), ConvertSecondsToFrameNumber(T1), bShouldTransact);
}

bool UAnimDataController::AddBoneCurve(FName BoneName, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddBoneTrack", "Adding Animation Data Track"), bShouldTransact);
	return InsertBoneTrack(BoneName, INDEX_NONE, bShouldTransact) != INDEX_NONE;
}

int32 UAnimDataController::InsertBoneTrack(FName BoneName, int32 DesiredIndex, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return INDEX_NONE;
	}
	
	const int32 TrackIndex = Model->GetBoneTrackIndexByName(BoneName);

	if (TrackIndex == INDEX_NONE)
	{
		if (Model->GetNumBoneTracks() >= MAX_ANIMATION_TRACKS)
		{
			ReportWarningf(LOCTEXT("MaxNumberOfTracksReachedWarning", "Cannot add track with name {0}. An animation sequence cannot contain more than 65535 tracks"), FText::FromName(BoneName));
			return INDEX_NONE;
		}
		else
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("InsertBoneTrack", "Inserting Animation Data Track"), bShouldTransact);

			// Determine correct index to do insertion at
			const int32 InsertIndex = Model->BoneAnimationTracks.IsValidIndex(DesiredIndex) ? DesiredIndex : Model->BoneAnimationTracks.Num();
			int32 BoneIndex = INDEX_NONE;

			if (const UAnimSequence* AnimationSequence = Model->GetAnimationSequence())
			{
				if (const USkeleton* Skeleton = AnimationSequence->GetSkeleton())
				{
					BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);

					if (BoneIndex == INDEX_NONE)
					{
						ReportErrorf(LOCTEXT("UnableToFindBoneIndexWarning", "Unable to retrieve bone index for track: {0}"), FText::FromName(BoneName));
					}
				}
				else
				{
					Report(ELogVerbosity::Error, LOCTEXT("UnableToGetOuterSkeletonError", "Unable to retrieve Skeleton for outer Animation Sequence"));
				}
			}
			else
			{
				Report(ELogVerbosity::Error, LOCTEXT("UnableToGetOuterAnimSequenceError", "Unable to retrieve outer Animation Sequence"));
			}

			if (BoneIndex != INDEX_NONE)
			{
				FBoneAnimationTrack& NewTrack = Model->BoneAnimationTracks.InsertDefaulted_GetRef(InsertIndex);
				NewTrack.Name = BoneName;
				NewTrack.BoneTreeIndex = BoneIndex;

				FAnimationTrackAddedPayload Payload;
				Payload.Name = BoneName;

				Model->GetNotifier().Notify<FAnimationTrackAddedPayload>(EAnimDataModelNotifyType::TrackAdded, Payload);
				ConditionalAction<UE::Anim::FRemoveTrackAction>(bShouldTransact, BoneName);

				return InsertIndex;
			}
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("TrackNameAlreadyExistsWarning", "Track with name {0} already exists"), FText::FromName(BoneName));
	}
	
	return TrackIndex;
}

bool UAnimDataController::RemoveBoneTrack(FName BoneName, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	const FBoneAnimationTrack* ExistingTrackPtr = Model->GetBoneAnimationTracks().FindByPredicate([BoneName](const FBoneAnimationTrack& SearchTrack) { return SearchTrack.Name == BoneName; });

	if (ExistingTrackPtr != nullptr)
	{
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveBoneTrack", "Removing Animation Data Track"), bShouldTransact);
		const int32 TrackIndex = Model->BoneAnimationTracks.IndexOfByPredicate([ExistingTrackPtr](const FBoneAnimationTrack& Track)
		{
			return Track.Name == ExistingTrackPtr->Name;
		});

		ensure(TrackIndex != INDEX_NONE);

		TArray<FTransform> BoneTransforms;
		Model->GetBoneTrackTransforms(BoneName, BoneTransforms);

		ConditionalAction<UE::Anim::FAddTrackAction>(bShouldTransact, BoneName, MoveTemp(BoneTransforms));
		Model->BoneAnimationTracks.RemoveAt(TrackIndex);

		FAnimationTrackRemovedPayload Payload;
		Payload.Name = BoneName;

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackRemoved, Payload);

		return true;
	}
	else
	{
		ReportWarningf(LOCTEXT("UnableToFindTrackWarning", "Could not find track with name {0}"), FText::FromName(BoneName));
	}

	return false;
}

void UAnimDataController::RemoveAllBoneTracks(bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return;
	}
	
	TArray<FName> TrackNames;
	Model->GetBoneTrackNames(TrackNames);

	if (TrackNames.Num())
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveAllBoneTracks", "Removing all Animation Data Tracks"), bShouldTransact);
		for (const FName& TrackName : TrackNames)
		{
			RemoveBoneTrack(TrackName, bShouldTransact);
		}
	}	
}

bool UAnimDataController::SetBoneTrackKeys(FName BoneName, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	// Validate key format
	const int32 MaxNumKeys = FMath::Max(FMath::Max(PositionalKeys.Num(), RotationalKeys.Num()), ScalingKeys.Num());

	if (MaxNumKeys > 0)
	{
		const bool bValidPosKeys = PositionalKeys.Num() == MaxNumKeys;
		const bool bValidRotKeys = RotationalKeys.Num() == MaxNumKeys;
		const bool bValidScaleKeys = ScalingKeys.Num() == MaxNumKeys;

		if (bValidPosKeys && bValidRotKeys && bValidScaleKeys)
		{
			if (FBoneAnimationTrack* TrackPtr = Model->FindMutableBoneTrackByName(BoneName))
			{
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetTrackKeysTransaction", "Setting Animation Data Track keys"), bShouldTransact);

				TArray<FTransform> BoneTransforms;
				Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
				ConditionalAction<UE::Anim::FSetTrackKeysAction>(bShouldTransact, BoneName, BoneTransforms);

				TrackPtr->InternalTrackData.PosKeys.SetNum(MaxNumKeys);
				TrackPtr->InternalTrackData.ScaleKeys.SetNum(MaxNumKeys);
				TrackPtr->InternalTrackData.RotKeys.SetNum(MaxNumKeys);
				for(int32 KeyIndex = 0; KeyIndex<MaxNumKeys; KeyIndex++)
				{
					TrackPtr->InternalTrackData.PosKeys[KeyIndex] = FVector3f(PositionalKeys[KeyIndex]);
					TrackPtr->InternalTrackData.ScaleKeys[KeyIndex] = FVector3f(ScalingKeys[KeyIndex]);
					TrackPtr->InternalTrackData.RotKeys[KeyIndex] = FQuat4f(RotationalKeys[KeyIndex]);
				}

				FAnimationTrackChangedPayload Payload;
				Payload.Name = BoneName;

				Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);

				return true;
			}
			else
			{
				ReportWarningf(LOCTEXT("InvalidTrackNameWarning", "Track with name {0} does not exist"), FText::FromName(BoneName));
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidTrackKeyDataError", "Invalid track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("MissingTrackKeyDataError", "Missing track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
	}

	return false;
}

bool UAnimDataController::SetBoneTrackKeys(FName BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact /*= true*/)
{
	if (!CheckOuterClass(UAnimSequence::StaticClass()))
	{
		return false;
	}

	FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetTrackKeysTransaction", "Setting Animation Data Track keys"), bShouldTransact);

	// Validate key format
	const int32 MaxNumKeys = FMath::Max(FMath::Max(PositionalKeys.Num(), RotationalKeys.Num()), ScalingKeys.Num());

	if (MaxNumKeys > 0)
	{
		const bool bValidPosKeys = PositionalKeys.Num() == MaxNumKeys;
		const bool bValidRotKeys = RotationalKeys.Num() == MaxNumKeys;
		const bool bValidScaleKeys = ScalingKeys.Num() == MaxNumKeys;

		if (bValidPosKeys && bValidRotKeys && bValidScaleKeys)
		{
			if (FBoneAnimationTrack* TrackPtr = Model->FindMutableBoneTrackByName(BoneName))
			{
				TArray<FTransform> BoneTransforms;
				Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
				ConditionalAction<UE::Anim::FSetTrackKeysAction>(bShouldTransact, BoneName, BoneTransforms);

				TrackPtr->InternalTrackData.PosKeys = PositionalKeys;
				TrackPtr->InternalTrackData.RotKeys = RotationalKeys;
				TrackPtr->InternalTrackData.ScaleKeys = ScalingKeys;

				FAnimationTrackChangedPayload Payload;
				Payload.Name = BoneName;

				Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);

				return true;
			}
			else
			{
				ReportWarningf(LOCTEXT("InvalidTrackNameWarning", "Track with name {0} does not exist"), FText::FromName(BoneName));
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidTrackKeyDataError", "Invalid track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("MissingTrackKeyDataError", "Missing track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
	}

	return false;
}

static int32 DiscreteInclusiveLower(const TRange<int32>& InRange)
{
	check(!InRange.GetLowerBound().IsOpen());

	// Add one for exclusive lower bounds since they start on the next subsequent frame
	static constexpr int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InRange.GetLowerBound().IsExclusive();

	return InRange.GetLowerBound().GetValue() + Offsets[OffsetIndex];
}

static int32 DiscreteExclusiveUpper(const TRange<int32>& InRange)
{
	check(!InRange.GetUpperBound().IsOpen());

	// Add one for inclusive upper bounds since they finish on the next subsequent frame
	static constexpr int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InRange.GetUpperBound().IsInclusive();

	return InRange.GetUpperBound().GetValue() + Offsets[OffsetIndex];
}

bool UAnimDataController::UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact)
{	
	if (!CheckOuterClass(UAnimSequence::StaticClass()))
	{
		return false;
	}
	// Validate key format
	const int32 MaxNumKeys = FMath::Max(FMath::Max(PositionalKeys.Num(), RotationalKeys.Num()), ScalingKeys.Num());
	const int32 RangeMin = DiscreteInclusiveLower(KeyRangeToSet);
	const int32 RangeMax = DiscreteExclusiveUpper(KeyRangeToSet);
	if (MaxNumKeys > 0)
	{
		const bool bValidPosKeys = PositionalKeys.Num() == MaxNumKeys;
		const bool bValidRotKeys = RotationalKeys.Num() == MaxNumKeys;
		const bool bValidScaleKeys = ScalingKeys.Num() == MaxNumKeys;

		if (bValidPosKeys && bValidRotKeys && bValidScaleKeys)
		{
			const int32 NumKeysToSet = RangeMax - RangeMin;
			if (NumKeysToSet == MaxNumKeys)
			{
				if (FBoneAnimationTrack* TrackPtr = Model->FindMutableBoneTrackByName(BoneName))
				{
					const FInt32Range TrackKeyRange(0, TrackPtr->InternalTrackData.PosKeys.Num());
					if(TrackKeyRange.Contains(KeyRangeToSet))
					{
						FRawAnimSequenceTrack& InternalTrackData = TrackPtr->InternalTrackData;
						TArray<FVector3f>& TrackPosKeys = InternalTrackData.PosKeys;
						TArray<FQuat4f>& TrackRotKeys = InternalTrackData.RotKeys;
						TArray<FVector3f>& TrackScaleKeys = InternalTrackData.ScaleKeys;

						FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetTrackKeysRangeTransaction", "Setting Animation Data Track keys"), bShouldTransact);

						TArray<FTransform> BoneTransforms;
						Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
						ConditionalAction<UE::Anim::FSetTrackKeysAction>(bShouldTransact, BoneName, BoneTransforms);

						int32 KeyIndex = 0;
						for (int32 FrameIndex = RangeMin; FrameIndex < RangeMax; ++FrameIndex, ++KeyIndex)
						{
							TrackPosKeys[FrameIndex] = PositionalKeys[KeyIndex];
							TrackRotKeys[FrameIndex] = RotationalKeys[KeyIndex];
							TrackScaleKeys[FrameIndex] = ScalingKeys[KeyIndex];
						}

						FAnimationTrackChangedPayload Payload;
						Payload.Name = BoneName;

						Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);

						return true;
					}
					else
					{
						ReportWarningf(LOCTEXT("InvalidTrackSetFrameRange", "Range of to-be-set bone track (with name {0}) keys does not overlap existing range"), FText::FromName(BoneName));
					}
				}
				else
				{
					ReportWarningf(LOCTEXT("InvalidTrackNameWarning", "Track with name {0} does not exist"), FText::FromName(BoneName));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("InvalidKeyIndexRangeError",
								 "Invalid key index range bound [{0}, {1}], expected to equal the size of the positional, rotational and scaling keys {2}"),
								 FText::AsNumber(RangeMin),
								 FText::AsNumber(RangeMax),
								 FText::AsNumber(MaxNumKeys));
			}	
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidTrackKeyDataError", "Invalid track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("MissingTrackKeyDataError", "Missing track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
	}

	return false;
}

bool UAnimDataController::UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact)
{
	if (!CheckOuterClass(UAnimSequence::StaticClass()))
	{
		return false;
	}

	// Validate key format
	const int32 MaxNumKeys = FMath::Max(FMath::Max(PositionalKeys.Num(), RotationalKeys.Num()), ScalingKeys.Num());
	const int32 RangeMin = DiscreteInclusiveLower(KeyRangeToSet);
	const int32 RangeMax = DiscreteExclusiveUpper(KeyRangeToSet);
	if (MaxNumKeys > 0)
	{
		const bool bValidPosKeys = PositionalKeys.Num() == MaxNumKeys;
		const bool bValidRotKeys = RotationalKeys.Num() == MaxNumKeys;
		const bool bValidScaleKeys = ScalingKeys.Num() == MaxNumKeys;

		if (bValidPosKeys && bValidRotKeys && bValidScaleKeys)
		{
			const int32 NumKeysToSet = RangeMax - RangeMin;
			if (NumKeysToSet == MaxNumKeys)
			{
				if (FBoneAnimationTrack* TrackPtr = Model->FindMutableBoneTrackByName(BoneName))
				{
					const FInt32Range TrackKeyRange(0, TrackPtr->InternalTrackData.PosKeys.Num());
					if(TrackKeyRange.Contains(KeyRangeToSet))
					{
						FRawAnimSequenceTrack& InternalTrackData = TrackPtr->InternalTrackData;
						TArray<FVector3f>& TrackPosKeys = InternalTrackData.PosKeys;
						TArray<FQuat4f>& TrackRotKeys = InternalTrackData.RotKeys;
						TArray<FVector3f>& TrackScaleKeys = InternalTrackData.ScaleKeys;

						FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetTrackKeysTransaction", "Setting Animation Data Track keys"), bShouldTransact);

						TArray<FTransform> BoneTransforms;
						Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
						ConditionalAction<UE::Anim::FSetTrackKeysAction>(bShouldTransact, BoneName, BoneTransforms);

						int32 KeyIndex = 0;
						for (int32 FrameIndex = RangeMin; FrameIndex < RangeMax; ++FrameIndex, ++KeyIndex)
						{
							TrackPosKeys[FrameIndex] = FVector3f(PositionalKeys[KeyIndex]);
							TrackRotKeys[FrameIndex] = FQuat4f(RotationalKeys[KeyIndex]);
							TrackScaleKeys[FrameIndex] = FVector3f(ScalingKeys[KeyIndex]);
						}

						FAnimationTrackChangedPayload Payload;
						Payload.Name = BoneName;

						Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);

						return true;
					}
					else
					{
						ReportWarningf(LOCTEXT("InvalidTrackSetFrameRange", "Range of to-be-set bone track (with name {0}) keys does not overlap existing range"), FText::FromName(BoneName));
					}
				}
				else
				{
					ReportWarningf(LOCTEXT("InvalidTrackNameWarning", "Track with name {0} does not exist"), FText::FromName(BoneName));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("InvalidKeyIndexRangeError",
								 "Invalid key index range bound [{0}, {1}], expected to equal the size of the positional, rotational and scaling keys {2}"),
								 FText::AsNumber(KeyRangeToSet.GetLowerBoundValue()),
								 FText::AsNumber(KeyRangeToSet.GetUpperBoundValue()),
								 FText::AsNumber(MaxNumKeys));
			}	
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidTrackKeyDataError", "Invalid track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("MissingTrackKeyDataError", "Missing track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
	}

	return false;
}

void UAnimDataController::ResizeCurves(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact /*= true*/)
{
	FBracket Bracket = ConditionalBracket(LOCTEXT("ResizeCurves", "Resizing all Curves"), bShouldTransact);

	for (FFloatCurve& Curve : Model->CurveData.FloatCurves)
	{
		FFloatCurve ResizedCurve = Curve;
		ResizedCurve.Resize(NewLength, bInserted, T0, T1);
		SetCurveKeys(FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Float), ResizedCurve.FloatCurve.GetConstRefOfKeys(), bShouldTransact);
	}

	for (FTransformCurve& Curve : Model->CurveData.TransformCurves)
	{
		FTransformCurve ResizedCurve = Curve;
		for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
		{
			const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
			FVectorCurve& SubCurve = *ResizedCurve.GetVectorCurveByIndex(SubCurveIndex);
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
				FAnimationCurveIdentifier TargetCurveIdentifier = FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Transform);
				UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
				
				FRichCurve& ChannelCurve = SubCurve.FloatCurves[ChannelIndex];
				ChannelCurve.ReadjustTimeRange(0, NewLength, bInserted, T0, T1);
				SetCurveKeys(TargetCurveIdentifier, ChannelCurve.GetConstRefOfKeys(), bShouldTransact);
			}
		}
	}
}

void UAnimDataController::ResizeAttributes(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact)
{
	FBracket Bracket = ConditionalBracket(LOCTEXT("ResizeAttributes", "Resizing all Attributes"), bShouldTransact);

	for (FAnimatedBoneAttribute& Attribute : Model->AnimatedBoneAttributes)
	{
		FAttributeCurve ResizedCurve = Attribute.Curve;
		ResizedCurve.ReadjustTimeRange(0, NewLength, bInserted, T0, T1);

		// Generate arrays necessary for API
		TArray<float> Times;
		TArray<const void*> Values;
		const TArray<FAttributeKey>& Keys = ResizedCurve.GetConstRefOfKeys();
		for (int32 KeyIndex = 0; KeyIndex < ResizedCurve.GetNumKeys(); ++KeyIndex)
		{
			Times.Add(Keys[KeyIndex].Time);
			Values.Add(Keys[KeyIndex].GetValuePtr<void>());
		}
		
		SetAttributeKeys(Attribute.Identifier, Times, Values, Attribute.Curve.GetScriptStruct(), bShouldTransact);
	}
}

bool UAnimDataController::ShouldTransact(bool bInLocalShouldTransact) const
{
	return bInLocalShouldTransact && (BracketDepth == 0 || bShouldTransactBracket);
}

bool UAnimDataController::AddAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		const FAnimatedBoneAttribute* const ExistingAttribute = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute) -> bool
			{
				return Attribute.Identifier.GetName() == AttributeIdentifier.GetName() && Attribute.Identifier.GetBoneName() == AttributeIdentifier.GetBoneName();
			});

		if (!ExistingAttribute)
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddAttribute", "Adding Animated Bone Attribute"), bShouldTransact);

			FAnimatedBoneAttribute& Attribute = Model->AnimatedBoneAttributes.AddDefaulted_GetRef();
			Attribute.Identifier = AttributeIdentifier;

			Attribute.Curve.SetScriptStruct(AttributeIdentifier.GetType());
		
			ConditionalAction<UE::Anim::FRemoveAtributeAction>(bShouldTransact, AttributeIdentifier);

			FAttributeAddedPayload Payload;
			Payload.Identifier = AttributeIdentifier;
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeAdded, Payload);

			return true;
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeAlreadyExists", "Attribute identifier with matching name already exists: {0} {1} ({2}) {3}"),
				FText::FromName(ExistingAttribute->Identifier.GetName()),
				FText::FromName(ExistingAttribute->Identifier.GetBoneName()),
				FText::AsNumber(ExistingAttribute->Identifier.GetBoneIndex()),
				FText::FromName(ExistingAttribute->Identifier.GetType()->GetFName()));
		}
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

bool UAnimDataController::RemoveAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		const int32 AttributeIndex = Model->AnimatedBoneAttributes.IndexOfByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute) -> bool
		{
			return Attribute.Identifier == AttributeIdentifier;
		});

		if (AttributeIndex != INDEX_NONE)
		{
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveAttribute", "Removing Animated Bone Attribute"), bShouldTransact);

			ConditionalAction<UE::Anim::FAddAtributeAction>(bShouldTransact, Model->AnimatedBoneAttributes[AttributeIndex]);

			Model->AnimatedBoneAttributes.RemoveAtSwap(AttributeIndex);
			
			FAttributeRemovedPayload Payload;
			Payload.Identifier = AttributeIdentifier;
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeRemoved, Payload);

			return true;
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
				FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
		}
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

int32 UAnimDataController::RemoveAllAttributesForBone(const FName& BoneName, bool bShouldTransact)
{
	int32 NumRemovedAttributes = 0;

	// Generate list of attribute identifiers, matching the bone name, for removal
	TArray<FAnimationAttributeIdentifier> Identifiers;
	Algo::TransformIf(Model->AnimatedBoneAttributes, Identifiers,
		[BoneName](const FAnimatedBoneAttribute& Attribute) -> bool
		{
			return Attribute.Identifier.GetBoneName() == BoneName;
		},
		[](const FAnimatedBoneAttribute& Attribute)
		{
			return Attribute.Identifier;
		}
	);

	if (Identifiers.Num())
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveAllAttributesForBone", "Removing all Attributes for Bone"), bShouldTransact);
		for (const FAnimationAttributeIdentifier& Identifier : Identifiers)
		{
			NumRemovedAttributes += RemoveAttribute(Identifier, bShouldTransact) ? 1 : 0;
		}
	}	

	return NumRemovedAttributes;
}


int32 UAnimDataController::RemoveAllAttributes(bool bShouldTransact)
{
	int32 NumRemovedAttributes = 0;

	TArray<FAnimationAttributeIdentifier> Identifiers;
	Algo::Transform(Model->AnimatedBoneAttributes, Identifiers, [](const FAnimatedBoneAttribute& Attribute)
	{
		return Attribute.Identifier;
	});

	if (Identifiers.Num())
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveAllAttributes", "Removing all Attributes"), bShouldTransact);
		for (const FAnimationAttributeIdentifier& Identifier : Identifiers)
		{
			NumRemovedAttributes += RemoveAttribute(Identifier, bShouldTransact) ? 1: 0;
		}
	}

	return NumRemovedAttributes;
}

bool UAnimDataController::SetAttributeKey_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		if (KeyValue)
		{
			FAnimatedBoneAttribute* AttributePtr = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
			{
				return Attribute.Identifier == AttributeIdentifier;
			});

			if (AttributePtr)
			{
				if (TypeStruct == AttributePtr->Identifier.GetType())
				{
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingAttributeKey", "Setting Animated Bone Attribute key"), bShouldTransact);

					FAttributeCurve& Curve = AttributePtr->Curve;
					const FKeyHandle KeyHandle = Curve.FindKey(Time);
					// In case the key does not yet exist one will be added, and thus the undo is a remove
					if (KeyHandle == FKeyHandle::Invalid())
					{
						ConditionalAction<UE::Anim::FRemoveAtributeKeyAction>(bShouldTransact, AttributeIdentifier, Time);
						Curve.UpdateOrAddTypedKey(Time, KeyValue, TypeStruct);
					}
					// In case the key does exist it will be updated , and thus the undo is a revert to the current value
					else
					{
						ConditionalAction<UE::Anim::FSetAtributeKeyAction>(bShouldTransact, AttributeIdentifier, Curve.GetKey(KeyHandle));
						Curve.UpdateOrAddTypedKey(Time, KeyValue, TypeStruct);
					}

					FAttributeChangedPayload Payload;
					Payload.Identifier = AttributeIdentifier;
					Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeChanged, Payload);

					return true;
				}
				else
				{
					ReportErrorf(LOCTEXT("AttributeTypeDoesNotMatchKeyType", "Key type does not match attribute: {0} {1}"),
						FText::FromName(AttributePtr->Identifier.GetType()->GetFName()), FText::FromName(TypeStruct->GetFName()));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
					FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
			}
		}
		else
		{
			Report(ELogVerbosity::Error, LOCTEXT("InvalidAttributeKey", "Invalid attribute key value provided"));
		}
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

bool UAnimDataController::SetAttributeKeys_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact)
{
	if (AttributeIdentifier.IsValid())
	{
		if (Times.Num() == KeyValues.Num())
		{
			FAnimatedBoneAttribute* AttributePtr = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
			{
				return Attribute.Identifier == AttributeIdentifier;
			});

			if (AttributePtr)
			{
				if (TypeStruct == AttributePtr->Identifier.GetType())
				{
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingAttributeKeys", "Setting Animated Bone Attribute keys"), bShouldTransact);

					FAnimatedBoneAttribute& Attribute = *AttributePtr;

					ConditionalAction<UE::Anim::FSetAtributeKeysAction>(bShouldTransact, Attribute);
			
					Attribute.Curve.SetKeys(Times, KeyValues);

					FAttributeChangedPayload Payload;
					Payload.Identifier = AttributeIdentifier;
					Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeChanged, Payload);

					return true;
				}
				else
				{
					ReportErrorf(LOCTEXT("AttributeTypeDoesNotMatchKeyType", "Key type does not match attribute: {0} {1}"),
						FText::FromName(AttributePtr->Identifier.GetType()->GetFName()), FText::FromName(TypeStruct->GetFName()));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
					FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeKeysMismatch", "Non matching number of key time/values: time entries {0} key entries {1}"),
				FText::AsNumber(Times.Num()), FText::AsNumber(KeyValues.Num()));
		}
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}


bool UAnimDataController::RemoveAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		FAnimatedBoneAttribute* AttributePtr = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
		{
			return Attribute.Identifier == AttributeIdentifier;
		});

		if (AttributePtr)
		{
			FAttributeCurve& Curve = AttributePtr->Curve;
			const FKeyHandle KeyHandle = Curve.FindKey(Time);

			if (KeyHandle != FKeyHandle::Invalid())
			{
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemovingAttributeKey", "Removing Animated Bone Attribute key"), bShouldTransact);

				ConditionalAction<UE::Anim::FAddAtributeKeyAction>(bShouldTransact, AttributeIdentifier, Curve.GetKey(KeyHandle));

				Curve.DeleteKey(KeyHandle);

				FAttributeAddedPayload Payload;
				Payload.Identifier = AttributeIdentifier;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeChanged, Payload);

				return true;
			}
			else
			{
				Report(ELogVerbosity::Warning, LOCTEXT("AttributeKeyNotFound", "Attribute does not contain key for provided time"));
			}	
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
				FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
		}
	}
	else
	{
		Report(ELogVerbosity::Error, LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

bool UAnimDataController::DuplicateAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, const FAnimationAttributeIdentifier& NewAttributeIdentifier, bool bShouldTransact)
{
	ValidateModel();

	if (AttributeIdentifier.IsValid() && NewAttributeIdentifier.IsValid())
	{
		if (AttributeIdentifier.GetType() == NewAttributeIdentifier.GetType())
		{
			const FAnimatedBoneAttribute* ExistingAttributePtr = Model->FindAttribute(NewAttributeIdentifier);
			if (ExistingAttributePtr == nullptr)
			{
				if(const FAnimatedBoneAttribute* AttributePtr = Model->FindAttribute(AttributeIdentifier))
				{
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("DuplicateAttribute", "Duplicating Animation Attribute"), bShouldTransact);

					FAnimatedBoneAttribute DuplicateAttribute;
					DuplicateAttribute.Identifier = NewAttributeIdentifier;
					DuplicateAttribute.Curve = AttributePtr->Curve;
					Model->AnimatedBoneAttributes.Add(DuplicateAttribute);

					FAttributeAddedPayload Payload;
					Payload.Identifier = NewAttributeIdentifier;
					Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeAdded, Payload);

					ConditionalAction<UE::Anim::FRemoveAtributeAction>(bShouldTransact, NewAttributeIdentifier);
					
					return true;
				}
				else
				{
					ReportWarningf(LOCTEXT("AttributeNameToDuplicateNotFoundWarning", "Could not find attribute with name {0} and type {1} for duplication"), FText::FromName(AttributeIdentifier.GetName()), FText::FromString(AttributeIdentifier.GetType()->GetName()));
				}
			}
			else
			{
				ReportWarningf(LOCTEXT("ExistingAttributeWarning", "Attribute with name {0} already exists"), FText::FromName(AttributeIdentifier.GetName()));
			}
		}
		else
		{
			ReportWarningf(LOCTEXT("InvalidAttributeTypeWarning", "Attribute types do not match: {0} ({1})"), FText::FromString(AttributeIdentifier.GetType()->GetName()), FText::FromString(AttributeIdentifier.GetType()->GetName()));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidAttributeIdentifierWarning", "Invalid attribute identifier(s) provided: {0} and/or {1}"), FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetName()));
	}

	return false;
}

void UAnimDataController::UpdateWithSkeleton(USkeleton* TargetSkeleton, bool bShouldTransact)
{
	OpenBracket(LOCTEXT("SettingNewskeleton", "Updating Skeleton for Animation Data Model"), bShouldTransact);
	{
		RemoveBoneTracksMissingFromSkeleton(TargetSkeleton, bShouldTransact);

		// Notify of skeleton change
		Model->GetNotifier().Notify(EAnimDataModelNotifyType::SkeletonChanged);
	}
	CloseBracket(bShouldTransact);
}

void UAnimDataController::PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> InModel)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Model->BoneAnimationTracks = InModel->GetBoneAnimationTracks();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Model->FrameRate = InModel->GetFrameRate();
	Model->NumberOfFrames = InModel->GetNumberOfFrames();
	Model->NumberOfKeys = InModel->GetNumberOfFrames() + 1;
	Model->CurveData = InModel->GetCurveData();
	Model->AnimatedBoneAttributes = InModel->GetAttributes();
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE // "AnimDataController"


