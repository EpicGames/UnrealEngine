// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDataControllerActions.h"
#include "AnimDataController.h"

#include "Animation/AnimData/IAnimationDataModel.h"
#include "UObject/StrongObjectPtr.h"

#define LOCTEXT_NAMESPACE "AnimDataControllerActions"

#if WITH_EDITOR

namespace UE {

namespace Anim {

TUniquePtr<FChange> FAnimDataBaseAction::Execute(UObject* Object)
{
	const TScriptInterface<IAnimationDataModel> DataModelInterface(Object);
	checkf(DataModelInterface, TEXT("Invalid IAnimationDataModel Object"));

	const TScriptInterface<IAnimationDataController> Controller = DataModelInterface->GetController();
	Controller->SetModel(DataModelInterface);

	return ExecuteInternal(DataModelInterface.GetInterface(), Controller.GetInterface());
}

FString FAnimDataBaseAction::ToString() const
{
	return ToStringInternal();
}

TUniquePtr<FChange> FOpenBracketAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	Controller->NotifyBracketOpen();
	return MakeUnique<FCloseBracketAction>(Description);
}

FString FOpenBracketAction::ToStringInternal() const
{
	return FString::Printf(TEXT("Open Bracket: %s"), *Description);
}

TUniquePtr<FChange> FCloseBracketAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	Controller->NotifyBracketClosed();
	return MakeUnique<FOpenBracketAction>(Description);
}

FString FCloseBracketAction::ToStringInternal() const
{
	return TEXT("Closing Bracket");
}

FAddTrackAction::FAddTrackAction(const FName& InName, TArray<FTransform>&& InTransformData)
{
	Name = InName;
	TransformData = MoveTemp(InTransformData);
}

TUniquePtr<FChange> FAddTrackAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	Controller->AddBoneCurve(Name, false);

	if (TransformData.Num())
	{
		TArray<FVector3f> PosKeys;
		TArray<FQuat4f> RotKeys;
		TArray<FVector3f> ScaleKeys;

		PosKeys.Reserve(TransformData.Num());
		RotKeys.Reserve(TransformData.Num());
		ScaleKeys.Reserve(TransformData.Num());	
		
		for (const FTransform& Transform : TransformData)
		{
			PosKeys.Add(FVector3f(Transform.GetLocation()));
			RotKeys.Add(FQuat4f(Transform.GetRotation()));
			ScaleKeys.Add(FVector3f(Transform.GetScale3D()));
		}
	
		Controller->SetBoneTrackKeys(Name, PosKeys, RotKeys, ScaleKeys, false);	
	}
	

	return MakeUnique<FRemoveTrackAction>(Name);
}

FString FAddTrackAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("AddTrackAction_Description", "Adding animation bone track '{0}'."), FText::FromName(Name)).ToString();
}

FRemoveTrackAction::FRemoveTrackAction(const FName& InName)
	: Name(InName)
{
}

TUniquePtr<FChange> FRemoveTrackAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	ensure(Model->IsValidBoneTrackName(Name));

	TArray<FTransform> BoneTransforms;
	Model->GetBoneTrackTransforms(Name, BoneTransforms);
	TUniquePtr<FChange> InverseAction = MakeUnique<FAddTrackAction>(Name, MoveTemp(BoneTransforms));

	Controller->RemoveBoneTrack(Name, false);
	return InverseAction;
}

FString FRemoveTrackAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("RemoveTrackAction_Description", "Removing animation bone Track '{0}'."), FText::FromName(Name)).ToString();
}

FSetTrackKeysAction::FSetTrackKeysAction(const FName& InName, TArray<FTransform>& InTransformData)
{
	Name = InName;
	TransformData = MoveTemp(InTransformData);
}

TUniquePtr<FChange> FSetTrackKeysAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TArray<FTransform> CurrentBoneTransforms;
	Model->GetBoneTrackTransforms(Name, CurrentBoneTransforms);
	TUniquePtr<FChange> InverseAction = MakeUnique<FSetTrackKeysAction>(Name, CurrentBoneTransforms);

	TArray<FVector3f> PosKeys;
	TArray<FQuat4f> RotKeys;
	TArray<FVector3f> ScaleKeys;

	PosKeys.Reserve(TransformData.Num());
	RotKeys.Reserve(TransformData.Num());
	ScaleKeys.Reserve(TransformData.Num());	
	
	for (const FTransform& Transform : TransformData)
	{
		PosKeys.Add(FVector3f(Transform.GetLocation()));
		RotKeys.Add(FQuat4f(Transform.GetRotation()));
		ScaleKeys.Add(FVector3f(Transform.GetScale3D()));
	}
	
	Controller->SetBoneTrackKeys(Name, PosKeys, RotKeys, ScaleKeys, false);
	return InverseAction;
}

FString FSetTrackKeysAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetTrackKeysAction_Description", "Setting keys for animation bone track '{0}'."), FText::FromName(Name)).ToString();
}

FResizePlayLengthInFramesAction::FResizePlayLengthInFramesAction(const IAnimationDataModel* InModel, FFrameNumber F0, FFrameNumber F1) : Frame0(F0), Frame1(F1)
{
	Length = InModel->GetNumberOfFrames();
}

TUniquePtr<FChange> FResizePlayLengthInFramesAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction = MakeUnique<FResizePlayLengthInFramesAction>(Model, Frame0, Frame1);
	Controller->ResizeNumberOfFrames(Length, Frame0, Frame1, false);
	return InverseAction;
}

FString FResizePlayLengthInFramesAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("ResizePlayLengthInFramesAction_Description", "Resizing play length to {0} frames."), FText::AsNumber(Length.Value)).ToString();
}

FSetFrameRateAction::FSetFrameRateAction(const IAnimationDataModel* InModel)
{
	FrameRate = InModel->GetFrameRate();
}

TUniquePtr<FChange> FSetFrameRateAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction = MakeUnique<FSetFrameRateAction>(Model);
	Controller->SetFrameRate(FrameRate, false);
	return InverseAction;
}

FString FSetFrameRateAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetFrameRateAction_Description", "Setting Frame Rate to {0}."), FrameRate.ToPrettyText()).ToString();
}

TUniquePtr<FChange> FAddCurveAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction = MakeUnique<FRemoveCurveAction>(CurveId);

	Controller->AddCurve(CurveId, Flags, false);

	return InverseAction;
}

FString FAddCurveAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("AddCurveAction_Description", "Adding {0} curve '{1}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FRemoveCurveAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction;	

	if (CurveId.CurveType == ERawCurveTrackTypes::RCT_Float)
	{
		const FFloatCurve& Curve = Model->GetFloatCurve(CurveId);
		InverseAction = MakeUnique<FAddFloatCurveAction>(CurveId, Curve.GetCurveTypeFlags(), Curve.FloatCurve.GetConstRefOfKeys(), Curve.Color);
	}
	else if (CurveId.CurveType == ERawCurveTrackTypes::RCT_Transform)
	{
		const FTransformCurve& Curve = Model->GetTransformCurve(CurveId);
		InverseAction = MakeUnique<FAddTransformCurveAction>(CurveId, Curve.GetCurveTypeFlags(), Curve);
	}

	Controller->RemoveCurve(CurveId, false);

	return InverseAction;
}

FString FRemoveCurveAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("RemoveCurveAction_Description", "Removing {0} curve '{1}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FSetCurveFlagsAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FAnimCurveBase& Curve = Model->GetCurve(CurveId);

	const int32 CurrentFlags = Curve.GetCurveTypeFlags();

	TUniquePtr<FChange> InverseAction = MakeUnique<FSetCurveFlagsAction>(CurveId, CurrentFlags, CurveType);
	
	Controller->SetCurveFlags(CurveId, Flags, false);

	return InverseAction;
}

FString FSetCurveFlagsAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));
	return FText::Format(LOCTEXT("SetCurveFlagsAction_Description", "Setting flags for {0} curve '{1}'."), FText::FromString(CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FRenameCurveAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	Controller->RenameCurve(CurveId, NewCurveId, false);
	return MakeUnique<FRenameCurveAction>(NewCurveId, CurveId);
}

FString FRenameCurveAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));
	return FText::Format(LOCTEXT("RenameCurveAction_Description", "Renaming {0} curve '{1}' to '{2}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName), FText::FromName(NewCurveId.CurveName)).ToString();
}

FScaleCurveAction::FScaleCurveAction(const FAnimationCurveIdentifier& InCurveId, float InOrigin, float InFactor, ERawCurveTrackTypes InCurveType) : CurveId(InCurveId), CurveType(InCurveType), Origin(InOrigin), Factor(InFactor)
{
	ensure(CurveType == ERawCurveTrackTypes::RCT_Float);
}

TUniquePtr<FChange> FScaleCurveAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const float InverseFactor = 1.0f / Factor;

	TUniquePtr<FChange> InverseAction = MakeUnique<FScaleCurveAction>(CurveId, Origin, InverseFactor, CurveType);

	Controller->ScaleCurve(CurveId, Origin, Factor, false);

	return InverseAction;
}

FString FScaleCurveAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("ScaleCurveAction_Description", "Scaling {0} curve '{1}'."), FText::FromString(CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FAddFloatCurveAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction = MakeUnique<FRemoveCurveAction>(CurveId);

	Controller->OpenBracket(LOCTEXT("AddFloatCurveAction_Description", "Adding float curve."), false);
	Controller->AddCurve(CurveId, Flags, false);
	Controller->SetCurveKeys(CurveId, Keys, false);
	Controller->SetCurveColor(CurveId, Color, false);
	Controller->CloseBracket(false);

	return InverseAction;
}

FString FAddFloatCurveAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("AddFloatCurveAction_Format", "Adding float curve '{0}'."), FText::FromName(CurveId.CurveName)).ToString();
}

FAddTransformCurveAction::FAddTransformCurveAction(const FAnimationCurveIdentifier& InCurveId, int32 InFlags, const FTransformCurve& InTransformCurve) : CurveId(InCurveId), Flags(InFlags)
{
	SubCurveKeys[0] = InTransformCurve.TranslationCurve.FloatCurves[0].GetConstRefOfKeys();
	SubCurveKeys[1] = InTransformCurve.TranslationCurve.FloatCurves[1].GetConstRefOfKeys();
	SubCurveKeys[2] = InTransformCurve.TranslationCurve.FloatCurves[2].GetConstRefOfKeys();

	SubCurveKeys[3] = InTransformCurve.RotationCurve.FloatCurves[0].GetConstRefOfKeys();
	SubCurveKeys[4] = InTransformCurve.RotationCurve.FloatCurves[1].GetConstRefOfKeys();
	SubCurveKeys[5] = InTransformCurve.RotationCurve.FloatCurves[2].GetConstRefOfKeys();

	SubCurveKeys[6] = InTransformCurve.ScaleCurve.FloatCurves[0].GetConstRefOfKeys();
	SubCurveKeys[7] = InTransformCurve.ScaleCurve.FloatCurves[1].GetConstRefOfKeys();
	SubCurveKeys[8] = InTransformCurve.ScaleCurve.FloatCurves[2].GetConstRefOfKeys();
}

TUniquePtr<FChange> FAddTransformCurveAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction = MakeUnique<FRemoveCurveAction>(CurveId);

	Controller->AddCurve(CurveId, Flags, false);

	for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
	{
		const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
		for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
		{
			const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
			FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
			UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
			Controller->SetCurveKeys(TargetCurveIdentifier, SubCurveKeys[(SubCurveIndex * 3) + ChannelIndex], false);
		}
	}

	return InverseAction;
}

FString FAddTransformCurveAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("AddTransformCurveAction_Description", "Adding transform curve '{0}'."), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FAddRichCurveKeyAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FChange> InverseAction = MakeUnique<FRemoveRichCurveKeyAction>(CurveId, Key.Time);
	Controller->SetCurveKey(CurveId, Key, false);
	return InverseAction;
}

FString FAddRichCurveKeyAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("AddNamedRichCurveKeyAction_Description", "Adding key to {0} curve '{1}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FSetRichCurveKeyAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FRichCurve& RichCurve = Model->GetRichCurve(CurveId);

	const FKeyHandle Handle = RichCurve.FindKey(Key.Time);
	ensure(Handle != FKeyHandle::Invalid());
	FRichCurveKey CurrentKey = RichCurve.GetKey(Handle);

	Controller->SetCurveKey(CurveId, Key, false);

	return MakeUnique<FSetRichCurveKeyAction>(CurveId, CurrentKey);
}

FString FSetRichCurveKeyAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("SetNamedRichCurveKeyAction_Description", "Setting key for {0} curve '{1}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FRemoveRichCurveKeyAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{	
	const FRichCurve& RichCurve = Model->GetRichCurve(CurveId);

	const FKeyHandle Handle = RichCurve.FindKey(Time);
	ensure(Handle != FKeyHandle::Invalid());
	FRichCurveKey CurrentKey = RichCurve.GetKey(Handle);

	Controller->RemoveCurveKey(CurveId, Time, false);

	return MakeUnique<FAddRichCurveKeyAction>(CurveId, CurrentKey);
}

FString FRemoveRichCurveKeyAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("RemoveNamedRichCurveKeyAction_Description", "Removing key from {0} curve '{1}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FSetRichCurveKeysAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FRichCurve& RichCurve = Model->GetRichCurve(CurveId);

	TUniquePtr<FChange> InverseAction = MakeUnique<FSetRichCurveKeysAction>(CurveId, RichCurve.GetConstRefOfKeys());

	Controller->SetCurveKeys(CurveId, Keys, false);

	return InverseAction;
}

FString FSetRichCurveKeysAction::ToStringInternal() const
{
	const FString FloatLabel(TEXT("float"));
	const FString TransformLabel(TEXT("transform"));

	return FText::Format(LOCTEXT("SetNamedRichCurveKeysAction_Description", "Replacing keys for {0} curve '{1}'."), FText::FromString(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float ? FloatLabel : TransformLabel), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FSetRichCurveAttributesAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FRichCurve& RichCurve = Model->GetRichCurve(CurveId);

	FCurveAttributes CurrentAttributes;
	CurrentAttributes.SetPreExtrapolation(RichCurve.PreInfinityExtrap);
	CurrentAttributes.SetPostExtrapolation(RichCurve.PostInfinityExtrap);

	TUniquePtr<FChange> InverseAction = MakeUnique<FSetRichCurveAttributesAction>(CurveId, CurrentAttributes);	
	Controller->SetCurveAttributes(CurveId, Attributes, false);
	
	return InverseAction;
	
}

FString FSetRichCurveAttributesAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetCurveAttributesAction_Description", "Setting curve attributes '{0}'."), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FSetCurveColorAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FAnimCurveBase& AnimationCurve = Model->GetCurve(CurveId);
	const FLinearColor CurrentColor = AnimationCurve.Color;

	Controller->SetCurveColor(CurveId, Color, false);

	return MakeUnique<FSetCurveColorAction>(CurveId, CurrentColor);
}

FString FSetCurveColorAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetCurveColorAction_Description", "Setting curve color '{0}'."), FText::FromName(CurveId.CurveName)).ToString();
}

TUniquePtr<FChange> FSetCurveCommentAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FAnimCurveBase& AnimationCurve = Model->GetCurve(CurveId);
	const FString& CurrentComment = AnimationCurve.Comment;

	Controller->SetCurveComment(CurveId, Comment, false);

	return MakeUnique<FSetCurveCommentAction>(CurveId, CurrentComment);
}

FString FSetCurveCommentAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetCurveCommentAction_Description", "Setting curve comment '{0}'."), FText::FromName(CurveId.CurveName)).ToString();
}

FAddAtributeAction::FAddAtributeAction(const FAnimatedBoneAttribute& InAttribute) : AttributeId(InAttribute.Identifier)
{
	Keys = InAttribute.Curve.GetConstRefOfKeys();
}

TUniquePtr<FChange> FAddAtributeAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{	
	TArray<const void*> VoidValues;
	Algo::Transform(Keys, VoidValues, [](const FAttributeKey& Key)
	{
		return Key.GetValuePtr<void>();
	});

	TArray<float> Times;
	Algo::Transform(Keys, Times, [](const FAttributeKey& Key)
	{
		return Key.Time;
	});

	Controller->AddAttribute(AttributeId, false);
	Controller->SetAttributeKeys(AttributeId, MakeArrayView(Times), MakeArrayView(VoidValues), AttributeId.GetType(), false);

	return MakeUnique<FRemoveAtributeAction>(AttributeId);
}

FString FAddAtributeAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("AddAttributeAction_Description", "Adding attribute '{0}'."), FText::FromName(AttributeId.GetName())).ToString();
}

TUniquePtr<FChange> FRemoveAtributeAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FAnimatedBoneAttribute& Attribute = Model->GetAttribute(AttributeId);
	TUniquePtr<FAddAtributeAction> InverseAction = MakeUnique<FAddAtributeAction>(Attribute);
	
	Controller->RemoveAttribute(AttributeId, false);

	return InverseAction;
}

FString FRemoveAtributeAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("RemoveAttributeAction_Description", "Removing attribute '{0}'."), FText::FromName(AttributeId.GetName())).ToString();
}

TUniquePtr<FChange> FAddAtributeKeyAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	Controller->SetAttributeKey(AttributeId, Key.Time, Key.GetValuePtr<void>(), AttributeId.GetType(), false);

	return MakeUnique<FRemoveAtributeKeyAction>(AttributeId, Key.Time);
}

FString FAddAtributeKeyAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("AddAttributeKeyAction_Description", "Adding key to attribute '{0}'."), FText::FromName(AttributeId.GetName())).ToString();
}

TUniquePtr<FChange> FSetAtributeKeyAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FAnimatedBoneAttribute& Attribute = Model->GetAttribute(AttributeId);

	const FKeyHandle Handle = Attribute.Curve.FindKey(Key.Time);
	ensure(Handle != FKeyHandle::Invalid());

	FAttributeKey CurrentKey = Attribute.Curve.GetKey(Handle);
	Controller->SetAttributeKey(AttributeId, Key.Time, Key.GetValuePtr<void>(), AttributeId.GetType(), false);

	return MakeUnique<FSetAtributeKeyAction>(AttributeId, CurrentKey);
}

FString FSetAtributeKeyAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetAttributeKeyAction_Description", "Setting key on attribute '{0}'."), FText::FromName(AttributeId.GetName())).ToString();
}

TUniquePtr<FChange> FRemoveAtributeKeyAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	const FAnimatedBoneAttribute& Attribute = Model->GetAttribute(AttributeId);

	const FKeyHandle Handle = Attribute.Curve.FindKey(Time);
	ensure(Handle != FKeyHandle::Invalid());

	FAttributeKey CurrentKey = Attribute.Curve.GetKey(Handle);

	Controller->RemoveAttributeKey(AttributeId, Time, false);

	return MakeUnique<FAddAtributeKeyAction>(AttributeId, CurrentKey);
}

FString FRemoveAtributeKeyAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("RemoveAttributeKeyAction_Description", "Removing key from attribute '{0}'."), FText::FromName(AttributeId.GetName())).ToString();
}

FSetAtributeKeysAction::FSetAtributeKeysAction(const FAnimatedBoneAttribute& InAttribute) : AttributeId(InAttribute.Identifier)
{
	Keys = InAttribute.Curve.GetConstRefOfKeys();
}

TUniquePtr<FChange> FSetAtributeKeysAction::ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller)
{
	TUniquePtr<FSetAtributeKeysAction> InverseAction = MakeUnique<FSetAtributeKeysAction>(Model->GetAttribute(AttributeId));

	TArray<const void*> VoidValues;
	Algo::Transform(Keys, VoidValues, [](const FAttributeKey& Key)
	{
		return Key.GetValuePtr<void>();
	});

	TArray<float> Times;
	Algo::Transform(Keys, Times, [](const FAttributeKey& Key)
	{
		return Key.Time;
	});

	Controller->SetAttributeKeys(AttributeId, MakeArrayView(Times), MakeArrayView(VoidValues), AttributeId.GetType(), false);

	return InverseAction;
}

FString FSetAtributeKeysAction::ToStringInternal() const
{
	return FText::Format(LOCTEXT("SetAttributeKeysAction_Description", "Replacing keys for attribute '{0}'."), FText::FromName(AttributeId.GetName())).ToString();
}

} // namespace Anim

} // namespace UE

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE // "AnimDataControllerActions"
