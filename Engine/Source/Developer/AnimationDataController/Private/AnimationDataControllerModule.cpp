// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAnimationDataControllerModule.h"
#include "AnimDataController.h"
#include "Modules/ModuleManager.h"

class FAnimationDataControllerModule : public IAnimationDataControllerModule
{
	virtual TScriptInterface<IAnimationDataController> GetController(UObject* Outer = GetTransientPackage()) override
	{
		check(Outer);
		UAnimDataController* AnimController = NewObject<UAnimDataController>(Outer);
		return TScriptInterface<IAnimationDataController>(AnimController);
	}
};

IMPLEMENT_MODULE(FAnimationDataControllerModule, AnimationDataController);