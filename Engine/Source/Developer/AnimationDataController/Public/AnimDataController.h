// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimCurveTypes.h"
#include "Algo/Transform.h"
#include "Animation/AnimData/IAnimationDataController.h"

#if WITH_EDITOR
#include "ChangeTransactor.h"
#endif // WITH_EDITOR

#include "AnimDataController.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UAnimDataController : public UObject, public IAnimationDataController
{
	GENERATED_BODY()

public:
	UAnimDataController() 
#if WITH_EDITOR
	: BracketDepth(0) 
#endif // WITH_EDITOR
	{}

#if WITH_EDITOR
	/** Begin IAnimationDataController overrides */
	ANIMATIONDATACONTROLLER_API virtual void SetModel(TScriptInterface<IAnimationDataModel> InModel) override;
    virtual TScriptInterface<IAnimationDataModel> GetModelInterface() const override { return ModelInterface; }
	virtual const IAnimationDataModel* const GetModel() const override { return Model.Get(); }
	ANIMATIONDATACONTROLLER_API virtual void OpenBracket(const FText& InTitle, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void CloseBracket(bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void SetNumberOfFrames(FFrameNumber Length, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void ResizeNumberOfFrames(FFrameNumber NewLength, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void ResizeInFrames(FFrameNumber Length, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void SetFrameRate(FFrameRate FrameRate, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void SetPlayLength(float Length, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void ResizePlayLength(float NewLength, float T0, float T1, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void Resize(float Length, float T0, float T1, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool AddBoneCurve(FName BoneName, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual int32 InsertBoneTrack(FName BoneName, int32 DesiredIndex, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool RemoveBoneTrack(FName BoneName, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void RemoveAllBoneTracks(bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetBoneTrackKeys(FName BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact = true) override;	
	ANIMATIONDATACONTROLLER_API virtual bool SetBoneTrackKeys(FName BoneName, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact = true) override;	
	ANIMATIONDATACONTROLLER_API virtual bool UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool AddCurve(const FAnimationCurveIdentifier& CurveId, int32 CurveFlags = 0x00000004, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool DuplicateCurve(const FAnimationCurveIdentifier& CopyCurveId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool RemoveCurve(const FAnimationCurveIdentifier& CurveId, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void RemoveAllCurvesOfType(ERawCurveTrackTypes SupportedCurveType, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveFlag(const FAnimationCurveIdentifier& CurveId, EAnimAssetCurveFlags Flag, bool bState = true, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveFlags(const FAnimationCurveIdentifier& CurveId, int32 Flags, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetTransformCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FTransform>& TransformValues, const TArray<float>& TimeKeys, bool bShouldTransact = true) override;	
	ANIMATIONDATACONTROLLER_API virtual bool SetTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, const FTransform& Value, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool RemoveTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool RenameCurve(const FAnimationCurveIdentifier& CurveToRenameId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveColor(const FAnimationCurveIdentifier& CurveId, FLinearColor Color, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveComment(const FAnimationCurveIdentifier& CurveId, const FString& Comment, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool ScaleCurve(const FAnimationCurveIdentifier& CurveId, float Origin, float Factor, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveKey(const FAnimationCurveIdentifier& CurveId, const FRichCurveKey& Key, bool bShouldTransact = true) override;	
	ANIMATIONDATACONTROLLER_API virtual bool RemoveCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FRichCurveKey>& CurveKeys, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool SetCurveAttributes(const FAnimationCurveIdentifier& CurveId, const FCurveAttributes& Attributes, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool RemoveBoneTracksMissingFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void UpdateAttributesFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void NotifyPopulated() override;
	ANIMATIONDATACONTROLLER_API virtual void ResetModel(bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual bool AddAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact = true) override;	
    ANIMATIONDATACONTROLLER_API virtual bool RemoveAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact = true) override;
    ANIMATIONDATACONTROLLER_API virtual int32 RemoveAllAttributesForBone(const FName& BoneName, bool bShouldTransact = true) override;
    ANIMATIONDATACONTROLLER_API virtual int32 RemoveAllAttributes(bool bShouldTransact = true) override;
	virtual bool SetAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact = true) override
	{
		return SetAttributeKey_Internal(AttributeIdentifier, Time, KeyValue, TypeStruct, bShouldTransact);
	}
	virtual bool SetAttributeKeys(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact = true) override
	{
		return SetAttributeKeys_Internal(AttributeIdentifier, Times, KeyValues, TypeStruct, bShouldTransact);
	}
    ANIMATIONDATACONTROLLER_API virtual bool RemoveAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, bool bShouldTransact = true) override;	
	ANIMATIONDATACONTROLLER_API virtual bool DuplicateAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, const FAnimationAttributeIdentifier& NewAttributeIdentifier, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void UpdateWithSkeleton(USkeleton* TargetSkeleton, bool bShouldTransact = true) override;
	ANIMATIONDATACONTROLLER_API virtual void PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> InModel) override;
	virtual void InitializeModel() override {}
protected:
	ANIMATIONDATACONTROLLER_API virtual void NotifyBracketOpen() override;
	ANIMATIONDATACONTROLLER_API virtual void NotifyBracketClosed() override;
	/** End IAnimationDataController overrides */
	
private:
	/** Internal functionality for setting Attribute curve key(s) */
	ANIMATIONDATACONTROLLER_API bool SetAttributeKey_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact = true);
	ANIMATIONDATACONTROLLER_API bool SetAttributeKeys_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact = true);

	/** Resizes the curve/attribute data stored on the model according to the provided new length and time at which to insert or remove time */
	ANIMATIONDATACONTROLLER_API void ResizeCurves(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact = true);
	ANIMATIONDATACONTROLLER_API void ResizeAttributes(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact = true);
	
	virtual bool ShouldTransact(bool bInLocalShouldTransact) const override;
#endif // WITH_EDITOR

private: 
#if WITH_EDITOR
	int32 BracketDepth;
	bool bShouldTransactBracket = false;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TWeakObjectPtr<UAnimDataModel> Model;
	
	UPROPERTY(transient)
	TScriptInterface<IAnimationDataModel> ModelInterface;
#endif // WITH_EDITORONLY_DATA

	friend class FAnimDataControllerTestBase;
	friend UE::Anim::FOpenBracketAction;
	friend UE::Anim::FCloseBracketAction;

	DECLARE_TRANSACTION_CONTROLLER
};
