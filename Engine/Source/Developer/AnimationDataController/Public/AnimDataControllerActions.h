// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveDataAbstraction.h"
#include "Misc/Change.h"
#include "Misc/CoreMiscDefines.h"

#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimTypes.h" 
#include "Misc/FrameRate.h"

#define UE_API ANIMATIONDATACONTROLLER_API

class UObject;
class IAnimationDataModel;
class IAnimationDataController;

#if WITH_EDITOR

namespace UE {

namespace Anim {

/**
* UAnimDataController instanced FChange-based objects used for storing mutations to an IAnimationDataModel within the Transaction Buffer.
* Each Action class represents an (invertable) operation mutating an IAnimationDataModel object utilizing a UAnimDataController. Allowing 
* for a more granular approach to undo/redo-ing changes while also allowing for script-based interoperability.
*/
class FAnimDataBaseAction : public FSwapChange
{
public:
	UE_API virtual TUniquePtr<FChange> Execute(UObject* Object) final;
	virtual ~FAnimDataBaseAction() = default;
	UE_API virtual FString ToString() const override;

protected:
	virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) = 0;
	virtual FString ToStringInternal() const = 0;
};

class FOpenBracketAction : public FAnimDataBaseAction
{
public:
	explicit FOpenBracketAction(const FString& InDescription) : Description(InDescription) {}
	virtual ~FOpenBracketAction() = default;
protected:
	FOpenBracketAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FString Description;
};

class FCloseBracketAction : public FAnimDataBaseAction
{
public:
	explicit FCloseBracketAction(const FString& InDescription) : Description(InDescription) {}
	virtual ~FCloseBracketAction() = default;
protected:
	FCloseBracketAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FString Description;
};

class FAddTrackAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FAddTrackAction(const FName& InName, TArray<FTransform>&& InTransformData);
	virtual ~FAddTrackAction() = default;
protected:
	FAddTrackAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FName Name;
	TArray<FTransform> TransformData;
};

class FRemoveTrackAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FRemoveTrackAction(const FName& InName);
	virtual ~FRemoveTrackAction() = default;
protected:
	FRemoveTrackAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FName Name;
};

class FSetTrackKeysAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FSetTrackKeysAction(const FName& InName, TArray<FTransform>& InTransformData);
	virtual ~FSetTrackKeysAction() = default;
protected:
	FSetTrackKeysAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FName Name;
	TArray<FTransform> TransformData;
};

class FResizePlayLengthInFramesAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FResizePlayLengthInFramesAction(const IAnimationDataModel* InModel, FFrameNumber F0, FFrameNumber F1);
	virtual ~FResizePlayLengthInFramesAction() override = default;
protected:
	FResizePlayLengthInFramesAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FFrameNumber Length;
	FFrameNumber Frame0;
	FFrameNumber Frame1;
};

class FSetFrameRateAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FSetFrameRateAction(const IAnimationDataModel* InModel);
	virtual ~FSetFrameRateAction() override = default;
protected:
	FSetFrameRateAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FFrameRate FrameRate;
};

class FAddCurveAction : public FAnimDataBaseAction
{
public:
	explicit FAddCurveAction(const FAnimationCurveIdentifier& InCurveId, int32 InFlags) : CurveId(InCurveId), Flags(InFlags) {}

	virtual ~FAddCurveAction() = default;
protected:
	FAddCurveAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	int32 Flags;
};

class FAddFloatCurveAction : public FAnimDataBaseAction
{
public:
	explicit FAddFloatCurveAction(const FAnimationCurveIdentifier& InCurveId, int32 InFlags, const TArray<FRichCurveKey>& InKeys, const FLinearColor& InColor) : CurveId(InCurveId), Flags(InFlags), Keys(InKeys), Color(InColor) {}
	virtual ~FAddFloatCurveAction() = default;
protected:
	FAddFloatCurveAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	int32 Flags;
	TArray<FRichCurveKey> Keys;
	FLinearColor Color;
};

class FAddTransformCurveAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FAddTransformCurveAction(const FAnimationCurveIdentifier& InCurveId, int32 InFlags, const FTransformCurve& InTransformCurve);
	virtual ~FAddTransformCurveAction() = default;
protected:
	FAddTransformCurveAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	int32 Flags;
	
	TArray<FRichCurveKey> SubCurveKeys[9];
};

class FRemoveCurveAction : public FAnimDataBaseAction
{
public:
	explicit FRemoveCurveAction(const FAnimationCurveIdentifier& InCurveId) : CurveId(InCurveId) {}
	virtual ~FRemoveCurveAction() = default;
protected:
	FRemoveCurveAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
};

class FSetCurveFlagsAction : public FAnimDataBaseAction
{
public:
	explicit FSetCurveFlagsAction(const FAnimationCurveIdentifier& InCurveId, int32 InFlags, ERawCurveTrackTypes InCurveType) : CurveId(InCurveId), Flags(InFlags), CurveType(InCurveType) {}
	virtual ~FSetCurveFlagsAction() = default;
protected:
	FSetCurveFlagsAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	int32 Flags;
	ERawCurveTrackTypes CurveType;
};

class FRenameCurveAction : public FAnimDataBaseAction
{
public:
	explicit FRenameCurveAction(const FAnimationCurveIdentifier& InCurveId, const FAnimationCurveIdentifier& InNewCurveId) : CurveId(InCurveId), NewCurveId(InNewCurveId) {}
	virtual ~FRenameCurveAction() = default;
protected:
	FRenameCurveAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	FAnimationCurveIdentifier NewCurveId;
};

class FScaleCurveAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FScaleCurveAction(const FAnimationCurveIdentifier& InCurveId, float InOrigin, float InFactor, ERawCurveTrackTypes InCurveType);
	virtual ~FScaleCurveAction() = default;
protected:
	FScaleCurveAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	ERawCurveTrackTypes CurveType;
	float Origin;
	float Factor;
};

class FAddRichCurveKeyAction : public FAnimDataBaseAction
{
public:
	explicit FAddRichCurveKeyAction(const FAnimationCurveIdentifier& InCurveId, const FRichCurveKey& InKey) : CurveId(InCurveId), Key(InKey) {}
	virtual ~FAddRichCurveKeyAction() = default;
protected:
	FAddRichCurveKeyAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	ERawCurveTrackTypes CurveType;
	FRichCurveKey Key;
};

class FSetRichCurveKeyAction : public FAnimDataBaseAction
{
public:
	explicit FSetRichCurveKeyAction(const FAnimationCurveIdentifier& InCurveId, const FRichCurveKey& InKey) : CurveId(InCurveId), Key(InKey) {}
	virtual ~FSetRichCurveKeyAction() = default;
protected:
	FSetRichCurveKeyAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	FRichCurveKey Key;
};

class FRemoveRichCurveKeyAction : public FAnimDataBaseAction
{
public:
	explicit FRemoveRichCurveKeyAction(const FAnimationCurveIdentifier& InCurveId, const float InTime) : CurveId(InCurveId), Time(InTime) {}
	virtual ~FRemoveRichCurveKeyAction() = default;
protected:
	FRemoveRichCurveKeyAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	float Time;
};

class FSetRichCurveKeysAction : public FAnimDataBaseAction
{
public:
	explicit FSetRichCurveKeysAction(const FAnimationCurveIdentifier& InCurveId, const TArray<FRichCurveKey>& InKeys) : CurveId(InCurveId), Keys(InKeys) {}
	virtual ~FSetRichCurveKeysAction() = default;
protected:
	FSetRichCurveKeysAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	TArray<FRichCurveKey> Keys;
};

class FSetRichCurveAttributesAction : public FAnimDataBaseAction
{
public:
	explicit FSetRichCurveAttributesAction(const FAnimationCurveIdentifier& InCurveId, const FCurveAttributes& InAttributes) : CurveId(InCurveId), Attributes(InAttributes) {}
	virtual ~FSetRichCurveAttributesAction() = default;
protected:
	FSetRichCurveAttributesAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	FCurveAttributes Attributes;
};

class FSetCurveColorAction : public FAnimDataBaseAction
{
public:
	explicit FSetCurveColorAction(const FAnimationCurveIdentifier& InCurveId, const FLinearColor& InColor) : CurveId(InCurveId), Color(InColor) {}
	virtual ~FSetCurveColorAction() = default;
protected:
	FSetCurveColorAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	FLinearColor Color;
};

class FSetCurveCommentAction : public FAnimDataBaseAction
{
public:
	explicit FSetCurveCommentAction(const FAnimationCurveIdentifier& InCurveId, const FString& InComment) : CurveId(InCurveId), Comment(InComment) {}
	virtual ~FSetCurveCommentAction() = default;
protected:
	FSetCurveCommentAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationCurveIdentifier CurveId;
	FString Comment;
};

class FAddAtributeAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FAddAtributeAction(const FAnimatedBoneAttribute& InAttribute);
	virtual ~FAddAtributeAction() = default;
protected:
	FAddAtributeAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationAttributeIdentifier AttributeId;
	TArray<FAttributeKey> Keys;
};

class FRemoveAtributeAction : public FAnimDataBaseAction
{
public:
	explicit FRemoveAtributeAction(const FAnimationAttributeIdentifier& InAttributeId) : AttributeId(InAttributeId) {}
	virtual ~FRemoveAtributeAction() = default;
protected:
	FRemoveAtributeAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationAttributeIdentifier AttributeId;
};

class FAddAtributeKeyAction : public FAnimDataBaseAction
{
public:
	explicit FAddAtributeKeyAction(const FAnimationAttributeIdentifier& InAttributeId, const FAttributeKey& InKey) : AttributeId(InAttributeId), Key(InKey) {}
	virtual ~FAddAtributeKeyAction() = default;
protected:
	FAddAtributeKeyAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationAttributeIdentifier AttributeId;
	FAttributeKey Key;
};

class FSetAtributeKeyAction : public FAnimDataBaseAction
{
public:
	explicit FSetAtributeKeyAction(const FAnimationAttributeIdentifier& InAttributeId, const FAttributeKey& InKey) : AttributeId(InAttributeId), Key(InKey) {}
	virtual ~FSetAtributeKeyAction() = default;
protected:
	FSetAtributeKeyAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationAttributeIdentifier AttributeId;
	FAttributeKey Key;
};

class FRemoveAtributeKeyAction : public FAnimDataBaseAction
{
public:
	explicit FRemoveAtributeKeyAction(const FAnimationAttributeIdentifier& InAttributeId, float InTime) : AttributeId(InAttributeId), Time(InTime) {}
	virtual ~FRemoveAtributeKeyAction() = default;
protected:
	FRemoveAtributeKeyAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationAttributeIdentifier AttributeId;
	float Time;
};

class FSetAtributeKeysAction : public FAnimDataBaseAction
{
public:
	UE_API explicit FSetAtributeKeysAction(const FAnimatedBoneAttribute& InAttribute);
	virtual ~FSetAtributeKeysAction() = default;
protected:
	FSetAtributeKeysAction() {}
	UE_API virtual TUniquePtr<FChange> ExecuteInternal(IAnimationDataModel* Model, IAnimationDataController* Controller) override;
	UE_API virtual FString ToStringInternal() const override;

protected:
	FAnimationAttributeIdentifier AttributeId;
	TArray<FAttributeKey> Keys;
};

} // namespace Anim

} // namespace UE

#endif // WITH_EDITOR

#undef UE_API
