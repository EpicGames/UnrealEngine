// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimData/IAnimationDataController.h"
#include "Modules/ModuleInterface.h"
#include "UObject/Package.h"

class UAnimSequence;
class UAnimSequenceBase;

class IAnimationDataControllerModule : public IModuleInterface
{
public:
	/** Returns UAnimationDataController instance, with optional outer, wrapped in a TScriptInterface of its implement IAnimationDataController interface */
	virtual TScriptInterface<IAnimationDataController> GetController(UObject* Outer = GetTransientPackage()) = 0;
};